// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class Horde : ModuleRules
{
	public Horde(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(new string[] { "Core", "HTTP", "Json" });

		PrivateIncludePathModuleNames.Add("DesktopPlatform");

//		bUseUnity = false;
	}
}
