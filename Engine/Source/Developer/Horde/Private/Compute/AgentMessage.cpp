// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/AgentMessage.h"
#include "Compute/ComputePlatform.h"
#include "../HordePlatform.h"

//// FAgentMessageChannel ////

FAgentMessageChannel::FAgentMessageChannel(TSharedPtr<FComputeChannel> InChannel)
	: ChannelBuffers(MoveTemp(InChannel))
	, RequestData(0)
	, RequestSize(0)
	, MaxRequestSize(0)
	, ResponseType(EAgentMessageType::None)
	, ResponseData(nullptr)
	, ResponseLength(0)
{
}

FAgentMessageChannel::~FAgentMessageChannel()
{
}

void FAgentMessageChannel::Close()
{
	CreateMessage(EAgentMessageType::None, 0);
	FlushMessage();
}

void FAgentMessageChannel::Ping()
{
	CreateMessage(EAgentMessageType::Ping, 0);
	FlushMessage();
}

void FAgentMessageChannel::Exception(const char* Description, const char* Trace)
{
	CreateMessage(EAgentMessageType::Exception, strlen(Description) + strlen(Trace) + 2);
	WriteString(Description);
	WriteString(Trace);
	FlushMessage();
}

void FAgentMessageChannel::Fork(int ChannelId, int BufferSize)
{
	CreateMessage(EAgentMessageType::Fork, sizeof(int) + sizeof(int));
	WriteInt32(ChannelId);
	WriteInt32(BufferSize);
	FlushMessage();
}

void FAgentMessageChannel::Attach()
{
	CreateMessage(EAgentMessageType::Attach, 0);
	FlushMessage();
}

void FAgentMessageChannel::UploadFiles(const char* Path, const char* Locator)
{
	CreateMessage(EAgentMessageType::WriteFiles, strlen(Path) + strlen(Locator) + 20);
	WriteString(Path);
	WriteString(Locator);
	FlushMessage();
}

void FAgentMessageChannel::DeleteFiles(const char* const * Paths, size_t Count)
{
	size_t RequiredSize = 10;
	for (size_t Idx = 0; Idx < Count; Idx++)
	{
		RequiredSize += strlen(Paths[Idx]) + 10;
	}

	CreateMessage(EAgentMessageType::DeleteFiles, RequiredSize);
	WriteUnsignedVarInt(Count);
	for (size_t Idx = 0; Idx < Count; Idx++)
	{
		WriteString(Paths[Idx]);
	}
	FlushMessage();
}

void FAgentMessageChannel::Execute(const char* Exe, const char* const * Args, size_t NumArgs, const char* WorkingDir, const char* const * EnvVars, size_t NumEnvVars, EExecuteProcessFlags Flags)
{
	size_t RequiredSize = 50 + strlen(Exe);
	for (size_t Idx = 0; Idx < NumArgs; Idx++)
	{
		RequiredSize += strlen(Args[Idx]) + 10;
	}
	if (WorkingDir != nullptr)
	{
		RequiredSize += strlen(WorkingDir) + 10;
	}
	for (size_t Idx = 0; Idx < NumEnvVars; Idx++)
	{
		RequiredSize += strlen(EnvVars[Idx]) + 20;
	}

	CreateMessage(EAgentMessageType::ExecuteV2, RequiredSize);
	WriteString(Exe);

	WriteUnsignedVarInt(NumArgs);
	for (size_t Idx = 0; Idx < NumArgs; Idx++)
	{
		WriteString(Args[Idx]);
	}

	WriteOptionalString(WorkingDir);

	WriteUnsignedVarInt(NumEnvVars);
	for (size_t Idx = 0; Idx < NumEnvVars; Idx++)
	{
		const char* EqualsPtr = strchr(EnvVars[Idx], '=');
		check(EqualsPtr != nullptr);

		WriteString(std::string_view(EnvVars[Idx], EqualsPtr - EnvVars[Idx]));
		if (*(EqualsPtr + 1) == 0)
		{
			WriteOptionalString(nullptr);
		}
		else
		{
			WriteOptionalString(EqualsPtr + 1);
		}
	}

	WriteInt32((int)Flags);
	FlushMessage();
}

void FAgentMessageChannel::Blob(const unsigned char* Data, size_t Length)
{
	const size_t MaxChunkSize = ChannelBuffers->Writer.GetChunkMaxLength() - 128 - MessageHeaderLength;
	for (size_t ChunkOffset = 0; ChunkOffset < Length;)
	{
		size_t ChunkLength = std::min(Length - ChunkOffset, MaxChunkSize);
		
		CreateMessage(EAgentMessageType::ReadBlobResponse, ChunkLength + 128);
		WriteInt32((int)ChunkOffset);
		WriteInt32((int)Length);
		WriteFixedLengthBytes(Data + ChunkOffset, ChunkLength);
		FlushMessage();

		ChunkOffset += ChunkLength;
	}
}

void FAgentMessageChannel::Xor(const unsigned char* Data, size_t Length, unsigned char Value)
{
	CreateMessage(EAgentMessageType::XorRequest, Length + 1);
	WriteFixedLengthBytes(Data, Length);
	WriteFixedLengthBytes(&Value, 1);
	FlushMessage();
}

EAgentMessageType FAgentMessageChannel::ReadResponse(int32 TimeoutMS, bool* bOutTimedOut)
{
	if (ResponseData)
	{
		ChannelBuffers->Reader.AdvanceReadPosition(ResponseLength + MessageHeaderLength);
		ResponseData = nullptr;
		ResponseLength = 0;
	}

	const unsigned char* Header = ChannelBuffers->Reader.WaitToRead(MessageHeaderLength, TimeoutMS, bOutTimedOut);
	if (Header == nullptr)
	{
		return EAgentMessageType::None;
	}
	unsigned int Length = *((unsigned int*)(Header + 1));

	Header = ChannelBuffers->Reader.WaitToRead(MessageHeaderLength + Length, TimeoutMS, bOutTimedOut);

	ResponseType = (EAgentMessageType)Header[0];
	ResponseData = Header + MessageHeaderLength;
	ResponseLength = Length;

	return ResponseType;
}

void FAgentMessageChannel::ReadException(AgentMessage::FException& Ex)
{
	check(ResponseType == EAgentMessageType::Exception);

	const unsigned char* Pos = ResponseData;
	Ex.Message = ReadString(&Pos);
	Ex.Description = ReadString(&Pos);
	check(Pos == ResponseData + ResponseLength);
}

int FAgentMessageChannel::ReadExecuteResult()
{
	check(ResponseType == EAgentMessageType::ExecuteResult);

	const unsigned char* Pos = ResponseData;
	int Result = ReadInt32(&Pos);
	check(Pos == ResponseData + ResponseLength);

	return Result;
}

void FAgentMessageChannel::ReadBlobRequest(AgentMessage::FBlobRequest& Ex)
{
	check(ResponseType == EAgentMessageType::ReadBlob);

	const unsigned char* Pos = ResponseData;
	Ex.Locator = ReadString(&Pos);
	Ex.Offset = (int)ReadUnsignedVarInt(&Pos);
	Ex.Length = (int)ReadUnsignedVarInt(&Pos);
	check(Pos == ResponseData + ResponseLength);
}

void FAgentMessageChannel::CreateMessage(EAgentMessageType Type, size_t MaxLength)
{
	RequestData = ChannelBuffers->Writer.WaitToWrite(MessageHeaderLength + MaxLength);
	RequestData[0] = (unsigned char)Type;
	MaxRequestSize = MaxLength;
}

void FAgentMessageChannel::FlushMessage()
{
	memcpy(&RequestData[1], &RequestSize, sizeof(int));
	ChannelBuffers->Writer.AdvanceWritePosition(MessageHeaderLength + RequestSize);
	RequestSize = 0;
	MaxRequestSize = 0;
	RequestData = nullptr;
}

void FAgentMessageChannel::WriteInt32(int Value)
{
	WriteFixedLengthBytes((const unsigned char*)&Value, sizeof(int));
}

int FAgentMessageChannel::ReadInt32(const unsigned char** Pos)
{
	int Value;
	memcpy(&Value, *Pos, sizeof(int));
	*Pos += sizeof(int);
	return Value;
}

void FAgentMessageChannel::WriteFixedLengthBytes(const unsigned char* Data, size_t Length)
{
	check(RequestSize + Length <= MaxRequestSize);
	memcpy(&RequestData[MessageHeaderLength + RequestSize], Data, Length);
	RequestSize += Length;
}

const unsigned char* FAgentMessageChannel::ReadFixedLengthBytes(const unsigned char** Pos, size_t Length)
{
	const unsigned char* Data = *Pos;
	*Pos += Length;
	return Data;
}

size_t FAgentMessageChannel::MeasureUnsignedVarInt(size_t Value)
{
	check(Value == (unsigned int)Value);

	if (Value == 0)
	{
		return 1;
	}
	else
	{
		return (FHordePlatform::FloorLog2((unsigned int)Value) / 7) + 1;
	}
}

void FAgentMessageChannel::WriteUnsignedVarInt(size_t Value)
{
	size_t ByteCount = MeasureUnsignedVarInt(Value);
	check(RequestSize + ByteCount <= MaxRequestSize);

	unsigned char* Output = RequestData + MessageHeaderLength + RequestSize;
	for (size_t Idx = 1; Idx < ByteCount; Idx++)
	{
		Output[ByteCount - Idx] = (unsigned)Value;
		Value >>= 8;
	}
	Output[0] = (unsigned char)((0xff << (9 - (int)ByteCount)) | (unsigned char)Value);

	RequestSize += ByteCount;
}

size_t FAgentMessageChannel::ReadUnsignedVarInt(const unsigned char** Pos)
{
	const unsigned char* Data = *Pos;

	unsigned char FirstByte = Data[0];
	size_t NumBytes = FHordePlatform::CountLeadingZeros(0xFF & (~(unsigned int)FirstByte)) + 1 - 24; // Note byte -> int conversion here, hence ignoring subtracting 24 bits

	size_t value = (size_t)(FirstByte & (0xff >> NumBytes));
	for (size_t Idx = 1; Idx < NumBytes; Idx++)
	{
		value <<= 8;
		value |= Data[Idx];
	}

	*Pos += NumBytes;
	return value;
}

size_t FAgentMessageChannel::MeasureString(const char* Text) const
{
	size_t Length = strlen(Text);
	return MeasureUnsignedVarInt(Length) + Length;
}

void FAgentMessageChannel::WriteString(const char* Text)
{
	size_t Length = strlen(Text);
	WriteUnsignedVarInt(Length);
	WriteFixedLengthBytes((const unsigned char*)Text, Length);
}

void FAgentMessageChannel::WriteString(const std::string_view& Text)
{
	WriteUnsignedVarInt(Text.size());
	WriteFixedLengthBytes((const unsigned char*)Text.data(), Text.size());
}

FUtf8StringView FAgentMessageChannel::ReadString(const unsigned char** Pos)
{
	size_t Length = ReadUnsignedVarInt(Pos);
	const char* Start = (const char*)ReadFixedLengthBytes(Pos, Length);
	return FUtf8StringView(Start, Length);
}

void FAgentMessageChannel::WriteOptionalString(const char* Text)
{
	if (Text == nullptr)
	{
		WriteUnsignedVarInt(0);
	}
	else
	{
		size_t Length = strlen(Text);
		WriteUnsignedVarInt(Length + 1);
		WriteFixedLengthBytes((const unsigned char*)Text, Length);
	}
}
