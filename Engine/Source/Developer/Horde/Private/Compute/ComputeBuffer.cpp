// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/ComputeBuffer.h"
#include "Compute/ComputePlatform.h"
#include "../HordePlatform.h"
#include <assert.h>
#include <iostream>
#include <assert.h>
#include <stdio.h>
#include <string.h>

struct FComputeBufferDetail
{
	static const int HeaderSize = (2 + FComputeBuffer::MaxReaders + FComputeBuffer::MaxChunks) * sizeof(long long);

	enum class EWriteState : unsigned long long
	{
		// Writer has moved to the next chunk
		MovedToNext = 0,

		// Chunk is still being appended to
		Writing = 2,

		// This chunk marks the end of the stream
		Complete = 3,
	};

	union FChunkState
	{
	public:
		long long Value;

		FChunkState()
			: Value(0)
		{ }

		FChunkState(long long InValue)
			: Value(InValue)
		{ }

		// Constructor
		FChunkState(EWriteState WriteState, int ReaderFlags, int Length)
			: Value((unsigned long long)Length | ((unsigned long long)ReaderFlags << 31) | ((unsigned long long)WriteState << 62))
		{ }

		// Written length of this chunk
		int GetLength() const { return (int)(Value & 0x7fffffff); }

		// Set of flags which are set for each reader that still has to read from a chunk
		int GetReaderFlags() const { return (int)((Value >> 31) & 0x7fffffff); }

		// State of the writer
		EWriteState GetWriteState() const { return (EWriteState)((unsigned long long)Value >> 62); }

		// Test whether a particular reader is still referencing the chunk
		bool HasReaderFlag(int ReaderIdx) const { return (Value & (1ULL << (31 + ReaderIdx))) != 0; }

	private:
		struct // For debugging purposes only; non-portable assumption of bitfield layout
		{
			unsigned long long Length : 31;
			unsigned long long ReaderFlags : 31;
			EWriteState WriteState : 2;
		} Fields;
	};

	struct FChunkStatePtr
	{
	public:
		FChunkStatePtr(volatile FChunkState* InChunkState)
			: State(InChunkState)
		{ }

		// Read the state value from memory
		FChunkState Get() const
		{
			return FChunkState(FPlatformAtomics::AtomicRead(&State->Value));
		}

		// Read the state value from memory
		void Set(FChunkState NewState)
		{
			FPlatformAtomics::AtomicStore(&State->Value, NewState.Value);
		}

		// Attempt to update the chunk state
		bool TryUpdate(FChunkState PrevState, FChunkState NextState)
		{
			return FPlatformAtomics::InterlockedCompareExchange(&State->Value, NextState.Value, PrevState.Value) == PrevState.Value;
		}

		// Append data to the chunk
		void Append(long long Length)
		{
			FPlatformAtomics::InterlockedAdd(&State->Value, Length);
		}

		// Mark this chunk as the end of the stream
		void MarkComplete()
		{
			FPlatformAtomics::InterlockedOr(&State->Value, FChunkState(EWriteState::Complete, 0, 0).Value);
		}

		// Start reading the chunk with the given reader
		void StartReading(int ReaderIdx)
		{
			FPlatformAtomics::InterlockedOr(&State->Value, FChunkState((EWriteState)0, 1 << ReaderIdx, 0).Value);
		}

		// Clear the reader flag
		void FinishReading(int ReaderIdx)
		{
			FPlatformAtomics::InterlockedAnd(&State->Value, ~FChunkState((EWriteState)0, 1 << ReaderIdx, 0).Value);
		}

		// Move to the next chunk
		void FinishWriting()
		{
			FPlatformAtomics::InterlockedAnd(&State->Value, ~FChunkState(EWriteState::Writing, 0, 0).Value);
		}

	private:
		volatile FChunkState* const State;
	};

	union FReaderState
	{
		long long Value;

		FReaderState()
			: Value(0)
		{ }

		FReaderState(long long InValue)
			: Value(InValue)
		{ }

		FReaderState(int ChunkIdx, int Offset, int RefCount, bool Detached)
			: Value((long long)(unsigned int)Offset | ((long long)(unsigned int)ChunkIdx << 32) | ((long long)(unsigned int)RefCount << 40) | (long long)(Detached ? (1ULL << 63) : 0))
		{ }

		int GetOffset() const
		{
			return (int)(Value & 0xffffffff);
		}

		int GetChunkIdx() const
		{
			return (int)((Value >> 32) & 0xff);
		}

		int GetRefCount() const
		{
			return (int)((Value >> 40) & 0x7fff);
		}

		bool IsDetached() const
		{
			return (Value & (1ULL << 63)) != 0;
		}

	private:
		struct // For debugging purposes only; non-portable assumption of bitfield layout
		{
			unsigned long long Offset : 32;
			unsigned long long ChunkIdx : 8;
			unsigned long long RefCount : 23;
			unsigned long long Detached : 1;
		} Fields;
	};

	struct FReaderStatePtr
	{
	public:
		FReaderStatePtr(volatile FReaderState* InReaderState)
			: State(InReaderState)
		{ }

		// Read the state value from memory
		FReaderState Get() const
		{
			return FReaderState(FPlatformAtomics::AtomicRead(&State->Value));
		}

		// Read the state value from memory
		void Set(FReaderState NewState)
		{
			FPlatformAtomics::AtomicStore(&State->Value, NewState.Value);
		}

		// Attempt to update the chunk state
		bool TryUpdate(FReaderState PrevState, FReaderState NextState)
		{
			return FPlatformAtomics::InterlockedCompareExchange(&State->Value, NextState.Value, PrevState.Value) == PrevState.Value;
		}

	private:
		volatile FReaderState* const State;
	};

	union FWriterState
	{
		long long Value;

		FWriterState()
			: Value(0)
		{ }

		FWriterState(long long InValue) 
			: Value(InValue) 
		{ }
		
		FWriterState(int ChunkIdx, int ReaderFlags, int RefCount, bool HasWrapped) 
			: Value((long long)(unsigned int)ChunkIdx | ((long long)(unsigned int)ReaderFlags << 32) | ((long long)(unsigned int)RefCount << 48) | (HasWrapped ? (1ULL << 63) : 0))
		{ }

		int GetChunkIdx() const { return Value & 0x7fffffff; }
		int GetReaderFlags() const { return ((unsigned long long)Value >> 32) & 0xffff; }
		int GetRefCount() const { return ((unsigned long long)Value >> 48) & 0x7fff; }
		bool HasWrapped() const { return (Value & (1ULL << 63)) != 0; }

	private:
		struct // For debugging purposes only; non-portable assumption of bitfield layout
		{
			int ChunkIdx : 32;
			int ReaderFlags : 8;
			int RefCount : 23;
			unsigned int Wrapped : 1;
		} Fields;
	};

	struct FWriterStatePtr
	{
	public:
		FWriterStatePtr(volatile FWriterState* InState)
			: State(InState)
		{ }

		FWriterState Get() const
		{
			return FWriterState(FPlatformAtomics::AtomicRead(&State->Value));
		}

		void Set(FWriterState NewState)
		{
			FPlatformAtomics::AtomicStore(&State->Value, NewState.Value);
		}

		bool TryUpdate(FWriterState PrevValue, FWriterState NextValue)
		{
			return FPlatformAtomics::InterlockedCompareExchange(&State->Value, NextValue.Value, PrevValue.Value) == PrevValue.Value;
		}

	private:
		volatile FWriterState* State;
	};

	struct FHeader
	{
	public:
		const unsigned short int NumReaders;
		const unsigned short int NumChunks;
		const int ChunkLength;

		FHeader(int InNumReaders, int InNumChunks, int InChunkLength)
			: NumReaders(InNumReaders)
			, NumChunks(InNumChunks)
			, ChunkLength(InChunkLength)
		{
			static_assert(sizeof(FWriterState) == sizeof(long long), "Incorrect size of FWriterState");
			static_assert(sizeof(FReaderState) == sizeof(long long), "Incorrect size of FReaderState");
			static_assert(sizeof(FChunkState) == sizeof(long long), "Incorrect size of FChunkState");
			static_assert(sizeof(FHeader) == HeaderSize, "Incorrect size of FHeader");

			GetChunkStatePtr(0).Set(FChunkState(EWriteState::Writing, 0, 0));
		}

		FWriterStatePtr GetWriterStatePtr()
		{
			return FWriterStatePtr(&Writer);
		}

		FReaderStatePtr GetReaderStatePtr(int ReaderIdx)
		{
			return FReaderStatePtr(&Readers[ReaderIdx]);
		}

		FChunkStatePtr GetChunkStatePtr(int ChunkIdx)
		{
			return FChunkStatePtr(&Chunks[ChunkIdx]);
		}

	private:
		FWriterState Writer;
		FReaderState Readers[FComputeBuffer::MaxReaders];
		FChunkState Chunks[FComputeBuffer::MaxChunks];
	};

	char Name[FComputeBuffer::MaxNameLength];

	FHeader* Header;
	unsigned char* ChunkPtrs[FComputeBuffer::MaxChunks];

	FComputeEvent WriterEvent;
	FComputeEvent ReaderEvents[FComputeBuffer::MaxReaders];

	int32 RefCount;

	FComputeBufferDetail(const char* InName)
		: Header(nullptr)
		, ChunkPtrs{ nullptr, }
		, RefCount(1)
	{
		static_assert(sizeof(FChunkStatePtr) == sizeof(long long), "Incorrect size of FChunkStatePtr; check union is declared correctly.");
		static_assert(sizeof(FWriterStatePtr) == sizeof(long long), "Incorrect size of FWriterStatePtr; check union is declared correctly.");

		FCStringAnsi::Strncpy(Name, InName, FComputeBuffer::MaxNameLength);
	}

	~FComputeBufferDetail()
	{
		Header = nullptr;
	}

	void AddRef()
	{
		FPlatformAtomics::InterlockedIncrement(&RefCount);
	}

	void Release()
	{
		if (FPlatformAtomics::InterlockedDecrement(&RefCount) == 0)
		{
			delete this;
		}
	}

	static TUniquePtr<FComputeBufferDetail> CreateNew(const FComputeBuffer::FParams& Params)
	{
		long long Capacity = sizeof(FHeader) + (Params.NumChunks * sizeof(unsigned int)) + (Params.NumChunks * Params.ChunkLength);

		const char* Name = Params.Name;

		char BaseNameBuffer[FComputeBuffer::MaxNameLength];
		if (Name == nullptr)
		{
			FHordePlatform::CreateUniqueName(BaseNameBuffer, FComputeBuffer::MaxNameLength);
			Name = BaseNameBuffer;
		}

		char NameBuffer[FComputeBuffer::MaxNameLength];
		snprintf(NameBuffer, FComputeBuffer::MaxNameLength, "%s_M", Name);

		TUniquePtr<FComputeBufferDetail> Detail = MakeUnique<FComputeBufferDetail>(Name);
		if (!Detail->MemoryMappedFile.Create(Params.Anonymous ? nullptr : NameBuffer, Capacity))
		{
			checkf(false, TEXT("Failed to create memory mapped file %hs"), NameBuffer);
			return nullptr;
		}

		void* Pointer = Detail->MemoryMappedFile.GetPointer();
		if (Pointer == nullptr)
		{
			checkf(false, TEXT("Failed to get pointer to memory mapping %hs"), NameBuffer);
			return nullptr;
		}

		Detail->Header = new(Pointer) FHeader(Params.NumReaders, Params.NumChunks, Params.ChunkLength);

		snprintf(NameBuffer, FComputeBuffer::MaxNameLength, "%s_W", Name);
		if (!Detail->WriterEvent.Create(Params.Anonymous ? nullptr : NameBuffer))
		{
			checkf(false, TEXT("Failed to create writer event %hs in memory mapping"), NameBuffer);
			return nullptr;
		}

		for (int ReaderIdx = 0; ReaderIdx < Detail->Header->NumReaders; ReaderIdx++)
		{
			snprintf(NameBuffer, FComputeBuffer::MaxNameLength, "%s_R%d", Name, ReaderIdx);
			if (!Detail->ReaderEvents[ReaderIdx].Create(Params.Anonymous ? nullptr : NameBuffer))
			{
				checkf(false, TEXT("Failed to create reader event %hs in memory mapping"), NameBuffer);
				return nullptr;
			}
		}

		InitShared(Detail.Get());
		return Detail;
	}

	static TUniquePtr<FComputeBufferDetail> OpenExisting(const char* Name)
	{
		char NameBuffer[FComputeBuffer::MaxNameLength];
		snprintf(NameBuffer, FComputeBuffer::MaxNameLength, "%s_M", Name);

		TUniquePtr<FComputeBufferDetail> Detail = MakeUnique<FComputeBufferDetail>(Name);
		if (!Detail->MemoryMappedFile.OpenExisting(NameBuffer))
		{
			return nullptr;
		}

		Detail->Header = (FHeader*)Detail->MemoryMappedFile.GetPointer();
		if (Detail->Header == nullptr)
		{
			return nullptr;
		}

		snprintf(NameBuffer, FComputeBuffer::MaxNameLength, "%s_W", Name);
		if (!Detail->WriterEvent.OpenExisting(NameBuffer))
		{
			return nullptr;
		}

		for (int ReaderIdx = 0; ReaderIdx < Detail->Header->NumReaders; ReaderIdx++)
		{
			snprintf(NameBuffer, FComputeBuffer::MaxNameLength, "%s_R%d", Name, ReaderIdx);
			if (!Detail->ReaderEvents[ReaderIdx].OpenExisting(NameBuffer))
			{
				return nullptr;
			}
		}

		InitShared(Detail.Get());
		return Detail;
	}

	int CreateReader()
	{
		for(int ReaderIdx = 0; ReaderIdx < Header->NumReaders; ReaderIdx++)
		{
			FReaderStatePtr ReaderStatePtr = Header->GetReaderStatePtr(ReaderIdx);
			for (;;)
			{
				FReaderState ReaderState = ReaderStatePtr.Get();
				if (ReaderState.GetRefCount() > 0)
				{
					break;
				}
				if (ReaderStatePtr.TryUpdate(ReaderState, FReaderState(0, 0, 1, false)))
				{
					FWriterStatePtr WriterStatePtr = Header->GetWriterStatePtr();
					for (;;)
					{
						FWriterState WriterState = WriterStatePtr.Get();
						check(!WriterState.HasWrapped());

						if (WriterStatePtr.TryUpdate(WriterState, FWriterState(WriterState.GetChunkIdx(), WriterState.GetReaderFlags() | (1 << ReaderIdx), WriterState.GetRefCount(), WriterState.HasWrapped())))
						{
							for (int WriteChunkIdx = 0; WriteChunkIdx <= WriterState.GetChunkIdx(); WriteChunkIdx++)
							{
								Header->GetChunkStatePtr(WriteChunkIdx).StartReading(ReaderIdx);
							}
							return ReaderIdx;
						}
					}
				}
			}
		}
		return -1;
	}

	void AddReaderRef(int ReaderIdx)
	{
		FReaderStatePtr ReaderStatePtr = Header->GetReaderStatePtr(ReaderIdx);
		for (; ; )
		{
			FReaderState ReaderState = ReaderStatePtr.Get();
			check(ReaderState.GetRefCount() > 0);

			if (ReaderStatePtr.TryUpdate(ReaderState, FReaderState(ReaderState.GetChunkIdx(), ReaderState.GetOffset(), ReaderState.GetRefCount() + 1, ReaderState.IsDetached())))
			{
				break;
			}
		}
	}

	void ReleaseReaderRef(int ReaderIdx)
	{
		FReaderStatePtr ReaderStatePtr = Header->GetReaderStatePtr(ReaderIdx);
		for (; ; )
		{
			FReaderState ReaderState = ReaderStatePtr.Get();
			check(ReaderState.GetRefCount() > 0);

			if (ReaderState.GetRefCount() == 1)
			{
				for (int Idx = 0; Idx < Header->NumChunks; Idx++)
				{
					Header->GetChunkStatePtr(Idx).FinishReading(ReaderIdx);
				}
			}

			if (ReaderStatePtr.TryUpdate(ReaderState, FReaderState(ReaderState.GetChunkIdx(), ReaderState.GetOffset(), ReaderState.GetRefCount() - 1, ReaderState.IsDetached())))
			{
				break;
			}
		}
	}

	void CreateWriter()
	{
		FWriterStatePtr WriterStatePtr = Header->GetWriterStatePtr();
		for (; ; )
		{
			FWriterState WriterState = WriterStatePtr.Get();
			int WriterStateRefCount = WriterState.GetRefCount();
			checkf(WriterStateRefCount == 0, TEXT("Attempting to CreateWriter with a non-zero WriterState refcount: %d"), WriterStateRefCount);

			if (WriterStatePtr.TryUpdate(WriterState, FWriterState(WriterState.GetChunkIdx(), WriterState.GetReaderFlags(), 1, WriterState.HasWrapped())))
			{
				FChunkStatePtr ChunkStatePtr = Header->GetChunkStatePtr(WriterState.GetChunkIdx());
				for (; ; )
				{
					FChunkState ChunkState = ChunkStatePtr.Get();
					if (ChunkStatePtr.TryUpdate(ChunkState, FChunkState(EWriteState::Writing, ChunkState.GetReaderFlags(), ChunkState.GetLength())))
					{
						break;
					}
				}
				break;
			}
		}
	}

	void AddWriterRef()
	{
		FWriterStatePtr WriterStatePtr = Header->GetWriterStatePtr();
		for (; ; )
		{
			FWriterState WriterState = WriterStatePtr.Get();
			check(WriterState.GetRefCount() > 0);

			if (WriterStatePtr.TryUpdate(WriterState, FWriterState(WriterState.GetChunkIdx(), WriterState.GetReaderFlags(), WriterState.GetRefCount() + 1, WriterState.HasWrapped())))
			{
				break;
			}
		}
	}

	void ReleaseWriterRef()
	{
		FWriterStatePtr writerStatePtr = Header->GetWriterStatePtr();
		for (; ; )
		{
			FWriterState writerState = writerStatePtr.Get();
			check(writerState.GetRefCount() > 0);

			if (writerState.GetRefCount() == 1)
			{
				MarkComplete();
			}

			if (writerStatePtr.TryUpdate(writerState, FWriterState(writerState.GetChunkIdx(), writerState.GetReaderFlags(), writerState.GetRefCount() - 1, writerState.HasWrapped())))
			{
				break;
			}
		}
	}

	bool IsComplete(int ReaderIdx) const
	{
		FReaderState ReaderState = Header->GetReaderStatePtr(ReaderIdx).Get();
		if (ReaderState.IsDetached())
		{
			return true;
		}

		FChunkState ChunkState = Header->GetChunkStatePtr(ReaderState.GetChunkIdx()).Get();
		return ChunkState.GetWriteState() == EWriteState::Complete && ReaderState.GetOffset() == ChunkState.GetLength();
	}

	void DetachReader(int ReaderIdx)
	{
		FReaderStatePtr ReaderStatePtr = Header->GetReaderStatePtr(ReaderIdx);
		for (; ; )
		{
			FReaderState ReaderState = ReaderStatePtr.Get();
			if (ReaderStatePtr.TryUpdate(ReaderState, FReaderState(ReaderState.GetChunkIdx(), ReaderState.GetOffset(), ReaderState.GetRefCount(), true)))
			{
				ReaderEvents[ReaderIdx].Signal();
				break;
			}
		}
	}

	void AdvanceReadPosition(int ReaderIdx, size_t Offset)
	{
		FReaderStatePtr ReaderStatePtr = Header->GetReaderStatePtr(ReaderIdx);
		for (; ; )
		{
			FReaderState ReaderState = ReaderStatePtr.Get();
			if (ReaderStatePtr.TryUpdate(ReaderState, FReaderState(ReaderState.GetChunkIdx(), ReaderState.GetOffset() + (int)Offset, ReaderState.GetRefCount(), ReaderState.IsDetached())))
			{
				ReaderEvents[ReaderIdx].Signal();
				break;
			}
		}
	}

	size_t GetMaxReadSize(int ReaderIdx) const
	{
		FReaderState ReaderState = Header->GetReaderStatePtr(ReaderIdx).Get();
		if (ReaderState.IsDetached())
		{
			return 0;
		}

		FChunkStatePtr ChunkStatePtr = Header->GetChunkStatePtr(ReaderState.GetChunkIdx());
		FChunkState ChunkState = ChunkStatePtr.Get();

		if (ChunkState.HasReaderFlag(ReaderIdx))
		{
			return ChunkState.GetLength() - ReaderState.GetOffset();
		}
		else
		{
			return 0;
		}
	}

	const unsigned char* WaitToRead(int ReaderIdx, size_t MinSize, int TimeoutMs, bool* bOutHasTimedOut)
	{
		for (; ; )
		{
			FReaderStatePtr ReaderStatePtr = Header->GetReaderStatePtr(ReaderIdx);
			FReaderState ReaderState = ReaderStatePtr.Get();
			if (ReaderState.IsDetached())
			{
				return nullptr;
			}

			FChunkStatePtr ChunkStatePtr = Header->GetChunkStatePtr(ReaderState.GetChunkIdx());
			FChunkState ChunkState = ChunkStatePtr.Get();

			if (!ChunkState.HasReaderFlag(ReaderIdx))
			{
				// Wait until the current chunk is readable
				if (!ReaderEvents[ReaderIdx].Wait(TimeoutMs))
				{
					if (bOutHasTimedOut)
					{
						*bOutHasTimedOut = true;
					}
					return nullptr;
				}
			}
			else if (ReaderState.GetOffset() + MinSize <= ChunkState.GetLength())
			{
				// We have enough data in the chunk to be able to read a message
				return ChunkPtrs[ReaderState.GetChunkIdx()] + ReaderState.GetOffset();
			}
			else if (ChunkState.GetWriteState() == EWriteState::Writing)
			{
				// Wait until there is more data in the chunk
				if (!ReaderEvents[ReaderIdx].Wait(TimeoutMs))
				{
					if (bOutHasTimedOut)
					{
						*bOutHasTimedOut = true;
					}
					return nullptr;
				}
			}
			else if (ReaderState.GetOffset() < ChunkState.GetLength() || ChunkState.GetWriteState() == EWriteState::Complete)
			{
				// Cannot read the requested amount of data from this chunk.
				return nullptr;
			}
			else if (ChunkState.GetWriteState() == EWriteState::MovedToNext)
			{
				// Move to the next chunk
				ChunkStatePtr.FinishReading(ReaderIdx);
				WriterEvent.Signal();

				int NextChunkIdx = ReaderState.GetChunkIdx() + 1;
				if (NextChunkIdx == Header->NumChunks)
				{
					NextChunkIdx = 0;
				}

				ReaderStatePtr.TryUpdate(ReaderState, FReaderState(NextChunkIdx, 0, ReaderState.GetRefCount(), ReaderState.IsDetached()));
			}
			else
			{
				check(false);
			}
		}
	}

	bool MarkComplete()
	{
		FWriterState WriterState = Header->GetWriterStatePtr().Get();

		FChunkStatePtr ChunkStatePtr = Header->GetChunkStatePtr(WriterState.GetChunkIdx());
		if (ChunkStatePtr.Get().GetWriteState() != EWriteState::Complete)
		{
			ChunkStatePtr.MarkComplete();
			SetAllReadEvents();
			return true;
		}

		return false;
	}

	void AdvanceWritePosition(size_t Size)
	{
		if (Size > 0)
		{
			FWriterState WriterState = Header->GetWriterStatePtr().Get();

			FChunkStatePtr ChunkStatePtr = Header->GetChunkStatePtr(WriterState.GetChunkIdx());
			FChunkState ChunkState = ChunkStatePtr.Get();

			check(ChunkState.GetWriteState() == EWriteState::Writing);
			ChunkStatePtr.Append(Size);

			SetAllReadEvents();
		}
	}

	size_t GetMaxWriteSize() const
	{
		FWriterState WriterState = Header->GetWriterStatePtr().Get();

		FChunkState ChunkState = Header->GetChunkStatePtr(WriterState.GetChunkIdx()).Get();
		if (ChunkState.GetWriteState() == EWriteState::Writing)
		{
			return Header->ChunkLength - ChunkState.GetLength();
		}
		else
		{
			return 0;
		}
	}

	size_t Write(const void* Buffer, size_t MaxSize, int TimeoutMs)
	{
		unsigned char* SendData = WaitToWrite(1, TimeoutMs);
		if (SendData == nullptr)
		{
			return 0;
		}

		size_t SendSize = GetMaxWriteSize();
		if (MaxSize < SendSize)
		{
			SendSize = MaxSize;
		}

		memcpy(SendData, Buffer, SendSize);
		AdvanceWritePosition(SendSize);
		return SendSize;
	}

	unsigned char* WaitToWrite(size_t MinSize, int TimeoutMs)
	{
		check(MinSize <= Header->ChunkLength);

		// Get the current chunk we're writing to
		FWriterState WriterState = Header->GetWriterStatePtr().Get();
		int WriteChunkIdx = WriterState.GetChunkIdx();

		FChunkStatePtr WriteChunkStatePtr = Header->GetChunkStatePtr(WriteChunkIdx);

		// Check if we can append to this chunk
		FChunkState ChunkState = WriteChunkStatePtr.Get();
		if (ChunkState.GetWriteState() == EWriteState::Writing)
		{
			int Length = ChunkState.GetLength();
			if (Length + MinSize <= Header->ChunkLength)
			{
				return ChunkPtrs[WriteChunkIdx] + Length;
			}

			WriteChunkStatePtr.FinishWriting(); // STATE CHANGE
			SetAllReadEvents();
		}

		if (ChunkState.GetWriteState() == EWriteState::Complete)
		{
			return nullptr;
		}

		// Otherwise get the next chunk to write to
		int NextWriteChunkIdx = WriteChunkIdx + 1;
		if (NextWriteChunkIdx == Header->NumChunks)
		{
			NextWriteChunkIdx = 0;
		}

		// Wait until all readers have finished with the chunk, and we can update the writer to match
		FChunkStatePtr NextWriteChunkStatePtr = Header->GetChunkStatePtr(NextWriteChunkIdx);
		for (;;)
		{
			FChunkState NextWriteChunkState = NextWriteChunkStatePtr.Get();
			if (NextWriteChunkState.GetReaderFlags() != 0)
			{
				if (!WriterEvent.Wait(TimeoutMs))
				{
					return nullptr;
				}
			}
			else if (NextWriteChunkStatePtr.TryUpdate(NextWriteChunkState, FChunkState(EWriteState::Writing, WriterState.GetReaderFlags(), 0)))
			{
				FWriterStatePtr WriterStatePtr = Header->GetWriterStatePtr();
				if (WriterStatePtr.TryUpdate(WriterState, FWriterState(NextWriteChunkIdx, WriterState.GetReaderFlags(), WriterState.GetRefCount(), NextWriteChunkIdx == 0)))
				{
					break;
				}
				else
				{
					WriterState = WriterStatePtr.Get();
				}
			}
		}
		return ChunkPtrs[NextWriteChunkIdx];
	}

private:
	FComputeMemoryMappedFile MemoryMappedFile;

	static void InitShared(FComputeBufferDetail* Detail)
	{
		FHeader* Header = Detail->Header;

		unsigned char* NextPtr = (unsigned char*)(Header + 1);
		for (int ChunkIdx = 0; ChunkIdx < Header->NumChunks; ChunkIdx++)
		{
			Detail->ChunkPtrs[ChunkIdx] = NextPtr;
			NextPtr += Header->ChunkLength;
		}
	}

	void SetAllReadEvents()
	{
		for (int Idx = 0; Idx < Header->NumReaders; Idx++)
		{
			ReaderEvents[Idx].Signal();
		}
	}
};








//// FComputeBuffer /////

FComputeBuffer::FComputeBuffer()
	: Detail(nullptr)
{
}

FComputeBuffer::FComputeBuffer(const FComputeBuffer& Other)
	: Detail(Other.Detail)
{
	if (Detail)
	{
		Detail->AddRef();
	}
}

FComputeBuffer::FComputeBuffer(FComputeBuffer&& Other) noexcept
	: Detail(Other.Detail)
{
	Other.Detail = nullptr;
}

FComputeBuffer::~FComputeBuffer()
{
	Close();
}

FComputeBuffer& FComputeBuffer::operator=(const FComputeBuffer& Other)
{
	Close();

	if (Other.Detail)
	{
		Detail = Other.Detail;
		Detail->AddRef();
	}

	return *this;
}

FComputeBuffer& FComputeBuffer::operator=(FComputeBuffer&& Other) noexcept
{
	Close();

	Detail = Other.Detail;
	Other.Detail = nullptr;

	return *this;
}

bool FComputeBuffer::CreateNew(const FParams& Params)
{
	Close();

	TUniquePtr<FComputeBufferDetail> Buffer = FComputeBufferDetail::CreateNew(Params);
	if (Buffer)
	{
		Detail = Buffer.Release();
		return true;
	}
	return false;
}

bool FComputeBuffer::OpenExisting(const char* Name)
{
	Close();

	TUniquePtr<FComputeBufferDetail> Buffer = FComputeBufferDetail::OpenExisting(Name);
	if (Buffer)
	{
		Detail = Buffer.Release();
		return true;
	}
	return false;
}

void FComputeBuffer::Close()
{
	if (Detail != nullptr)
	{
		Detail->Release();
		Detail = nullptr;
	}
}

FComputeBufferReader FComputeBuffer::CreateReader()
{
	int ReaderIdx = Detail->CreateReader();
	Detail->AddRef();
	return FComputeBufferReader(Detail, ReaderIdx);
}

FComputeBufferWriter FComputeBuffer::CreateWriter()
{
	Detail->CreateWriter();
	Detail->AddRef();
	return FComputeBufferWriter(Detail);
}

const char* FComputeBuffer::GetName() const
{
	return Detail->Name;
}






//// FComputeBufferReader /////

FComputeBufferReader::FComputeBufferReader()
	: Detail(nullptr)
	, ReaderIdx(0)
{
}

FComputeBufferReader::FComputeBufferReader(const FComputeBufferReader& Other)
	: Detail(Other.Detail)
	, ReaderIdx(Other.ReaderIdx)
{
	if (Detail)
	{
		Detail->AddRef();
		Detail->AddReaderRef(ReaderIdx);
	}
}

FComputeBufferReader::FComputeBufferReader(FComputeBufferReader&& Other) noexcept
	: Detail(Other.Detail)
	, ReaderIdx(Other.ReaderIdx)
{
	Other.Detail = nullptr;
}

FComputeBufferReader::~FComputeBufferReader()
{
	Close();
}

FComputeBufferReader& FComputeBufferReader::operator=(const FComputeBufferReader& Other)
{
	Close();

	if (Other.Detail)
	{
		Detail = Other.Detail;
		ReaderIdx = Other.ReaderIdx;

		Detail->AddRef();
		Detail->AddReaderRef(ReaderIdx);
	}

	return *this;
}

FComputeBufferReader& FComputeBufferReader::operator=(FComputeBufferReader&& Other) noexcept
{
	Close();

	Detail = Other.Detail;
	ReaderIdx = Other.ReaderIdx;

	Other.Detail = nullptr;
	return *this;
}

void FComputeBufferReader::Close()
{
	if (Detail != nullptr)
	{
		Detail->ReleaseReaderRef(ReaderIdx);
		Detail->Release();
		Detail = nullptr;
	}
}

void FComputeBufferReader::Detach()
{
	check(Detail);
	Detail->DetachReader(ReaderIdx);
}

bool FComputeBufferReader::IsComplete() const
{
	check(Detail);
	return Detail->IsComplete(ReaderIdx);
}

void FComputeBufferReader::AdvanceReadPosition(size_t Size)
{
	check(Detail);
	Detail->AdvanceReadPosition(ReaderIdx, Size);
}

size_t FComputeBufferReader::GetMaxReadSize() const
{
	check(Detail);
	return Detail->GetMaxReadSize(ReaderIdx);
}

size_t FComputeBufferReader::Read(void* Buffer, size_t MaxSize, int TimeoutMs, bool* bOutTimedOut)
{
	const unsigned char* RecvData = WaitToRead(1, TimeoutMs, bOutTimedOut);
	if (RecvData == nullptr)
	{
		return 0;
	}

	size_t RecvSize = GetMaxReadSize();
	if (MaxSize < RecvSize)
	{
		RecvSize = MaxSize;
	}

	memcpy(Buffer, RecvData, RecvSize);
	AdvanceReadPosition(RecvSize);
	return RecvSize;
}

const unsigned char* FComputeBufferReader::WaitToRead(size_t MinSize, int TimeoutMs, bool* bOutTimedOut)
{
	check(Detail);
	return Detail->WaitToRead(ReaderIdx, MinSize, TimeoutMs, bOutTimedOut);
}

FComputeBufferReader::FComputeBufferReader(FComputeBufferDetail* InDetail, int InReaderIdx)
	: Detail(InDetail)
	, ReaderIdx(InReaderIdx)
{
}

const char* FComputeBufferReader::GetName() const
{
	check(Detail);
	return Detail->Name;
}





//// FComputeBufferWriter /////

FComputeBufferWriter::FComputeBufferWriter()
	: Detail(nullptr)
{
}

FComputeBufferWriter::FComputeBufferWriter(const FComputeBufferWriter& Other)
	: Detail(Other.Detail)
{
	if (Detail)
	{
		Detail->AddRef();
		Detail->AddWriterRef();
	}
}

FComputeBufferWriter::FComputeBufferWriter(FComputeBufferWriter&& Other) noexcept
	: Detail(Other.Detail)
{
	Other.Detail = nullptr;
}

FComputeBufferWriter::~FComputeBufferWriter()
{
	Close();
}

FComputeBufferWriter& FComputeBufferWriter::operator=(const FComputeBufferWriter& Other)
{
	Close();

	if (Other.Detail)
	{
		Detail = Other.Detail;
		Detail->AddWriterRef();
		Detail->AddRef();
	}

	return *this;
}

FComputeBufferWriter& FComputeBufferWriter::operator=(FComputeBufferWriter&& Other) noexcept
{
	Close();

	Detail = Other.Detail;
	Other.Detail = nullptr;

	return *this;
}


void FComputeBufferWriter::Close()
{
	if (Detail != nullptr)
	{
		Detail->ReleaseWriterRef();
		Detail->Release();
		Detail = nullptr;
	}
}

void FComputeBufferWriter::MarkComplete()
{
	Detail->MarkComplete();
}

void FComputeBufferWriter::AdvanceWritePosition(size_t Size)
{
	Detail->AdvanceWritePosition(Size);
}

size_t FComputeBufferWriter::GetMaxWriteSize() const
{
	return Detail->GetMaxWriteSize();
}

size_t FComputeBufferWriter::GetChunkMaxLength() const
{
	return Detail->Header->ChunkLength;
}

size_t FComputeBufferWriter::Write(const void* Buffer, size_t MaxSize, int TimeoutMs)
{
	unsigned char* SendData = WaitToWrite(1, TimeoutMs);
	if (SendData == nullptr)
	{
		return 0;
	}

	size_t SendSize = GetMaxWriteSize();
	if (MaxSize < SendSize)
	{
		SendSize = MaxSize;
	}

	memcpy(SendData, Buffer, SendSize);
	AdvanceWritePosition(SendSize);
	return SendSize;
}

unsigned char* FComputeBufferWriter::WaitToWrite(size_t MinSize, int TimeoutMs)
{
	return Detail->WaitToWrite(MinSize, TimeoutMs);
}

FComputeBufferWriter::FComputeBufferWriter(FComputeBufferDetail* InDetail)
	: Detail(InDetail)
{
}

const char* FComputeBufferWriter::GetName() const
{
	return Detail->Name;
}
