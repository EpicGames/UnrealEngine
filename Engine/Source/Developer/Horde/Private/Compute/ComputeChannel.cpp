// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/ComputeChannel.h"
#include <string.h>
#include <stdio.h>
#include "Compute/ComputeSocket.h"

FComputeChannel::FComputeChannel()
{
}

FComputeChannel::FComputeChannel(FComputeBufferReader InReader, FComputeBufferWriter InWriter)
	: Reader(MoveTemp(InReader))
	, Writer(MoveTemp(InWriter))
{
}

FComputeChannel::~FComputeChannel()
{
}

bool FComputeChannel::IsValid() const
{
	return Reader.IsValid();
}

size_t FComputeChannel::Send(const void* Data, size_t Size, int TimeoutMs)
{
	return Writer.Write(Data, Size, TimeoutMs);
}

size_t FComputeChannel::Recv(void* Data, size_t Size, int TimeoutMs)
{
	return Reader.Read(Data, Size, TimeoutMs);
}

void FComputeChannel::MarkComplete()
{
	Writer.MarkComplete();
}
