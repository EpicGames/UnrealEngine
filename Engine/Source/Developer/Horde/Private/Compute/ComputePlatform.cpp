// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/ComputePlatform.h"
#include "HAL/Event.h"
#include <assert.h>
#include <wchar.h>
#include <bit>
#include <algorithm>
#include <iostream>

// Defines for the current platform
#ifdef _MSC_VER
	#define UE_COMPUTE_PLATFORM_WINDOWS 1
	#define UE_COMPUTE_PLATFORM_MAC 0
	#define UE_COMPUTE_PLATFORM_LINUX 0
#elif defined(__APPLE__)
	#define UE_COMPUTE_PLATFORM_WINDOWS 0
	#define UE_COMPUTE_PLATFORM_MAC 1
	#define UE_COMPUTE_PLATFORM_LINUX 0
#else
	#define UE_COMPUTE_PLATFORM_WINDOWS 0
	#define UE_COMPUTE_PLATFORM_MAC 0
	#define UE_COMPUTE_PLATFORM_LINUX 1
#endif

#if UE_COMPUTE_PLATFORM_WINDOWS
	#include "Windows/WindowsHWrapper.h"
	#undef min
	#undef max
	#undef GetEnvironmentVariable
	#undef SendMessage
	#undef InterlockedIncrement
#else
	#include <semaphore.h>
	#include <unistd.h>
	#include <atomic>
	#include <sys/mman.h>
	#include <sys/stat.h>
	#include <time.h>
	#include <fcntl.h>

	struct FFormatErrno { };

	static std::ostream &operator<<(std::ostream &os, const FFormatErrno &)
	{
		os << "(errno=" << errno << ": " << strerror(errno) << ")";
		return os;
	}
#endif

///////////////////////////////////////////////////

FComputeEvent::FComputeEvent()
	: Handle(nullptr)
	, Anonymous(false)
{
}

FComputeEvent::~FComputeEvent()
{
	Close();
}

bool FComputeEvent::Create(const char* Name)
{
	Close();

	Anonymous = Name == nullptr;

	if (Anonymous)
	{
		new (&Handle) FEventRef();
		return true;
	}


#if UE_COMPUTE_PLATFORM_WINDOWS
	Handle = CreateEventA(NULL, false, false, Name);
	return Handle != nullptr;
#else
	sem_t* Value = sem_open(Name, O_CREAT | O_EXCL, 0666, 1);
	if(Value != SEM_FAILED)
	{
		Handle = Value;
		return true;
	}
	checkf(false, TEXT("sem_open to create semaphore with name %hs failed (%hs)"), Name, strerror(errno));
	return false;
#endif
}

bool FComputeEvent::OpenExisting(const char* Name)
{
	Close();

	check(Name);
	Anonymous = false;

#if UE_COMPUTE_PLATFORM_WINDOWS
	Handle = OpenEventA(SYNCHRONIZE | EVENT_MODIFY_STATE, true, Name);
	return Handle != nullptr;
#else
	sem_t* Value = sem_open(Name, 0);
	if(Value != SEM_FAILED)
	{
		Handle = Value;
		return true;
	}
	return false;
#endif
}

void FComputeEvent::Close()
{
	if (Anonymous)
	{
		((FEventRef*)&Handle)->~FEventRef();
		Handle = nullptr;
		return;
	}

#if UE_COMPUTE_PLATFORM_WINDOWS
	if (Handle != nullptr)
	{
		CloseHandle(Handle);
		Handle = nullptr;
	}
#else
	if (Handle != nullptr)
	{
		sem_close((sem_t*)Handle);
		Handle = nullptr;
	}
#endif
}

void FComputeEvent::Signal()
{
	if (Anonymous)
	{
		(*((FEventRef*)&Handle))->Trigger();
		return;
	}

#if UE_COMPUTE_PLATFORM_WINDOWS
	SetEvent(Handle);
#else
	sem_post((sem_t*)Handle);
#endif
}

bool FComputeEvent::Wait(int timeoutMs)
{
	if (Anonymous)
	{
		return (*((FEventRef*)&Handle))->Wait(timeoutMs == -1 ? MAX_uint32 : timeoutMs);
	}

#if UE_COMPUTE_PLATFORM_WINDOWS
	DWORD WaitParam = (timeoutMs < 0) ? INFINITE : (DWORD)timeoutMs;
	return WaitForSingleObject(Handle, WaitParam) != WAIT_TIMEOUT;
#else
	if (timeoutMs == -1)
	{
		return sem_wait((sem_t*)Handle) == 0;
	}

	if (sem_trywait((sem_t*)Handle) == 0)
	{
		return true;
	}

	if(timeoutMs == 0)
	{
		return false;
	}
		
	struct timespec ts;
	if (clock_gettime(CLOCK_REALTIME, &ts) == -1)
	{
		check(false);
		return false;
	}
	
	const long long NsPerSec = 1'000'000'000;
	const long long NsPerMs = NsPerSec / 1000;
	
	long long newNs = (long long)ts.tv_nsec + (timeoutMs * NsPerMs);
	ts.tv_nsec = newNs % NsPerSec;
	ts.tv_sec += newNs / NsPerSec;
	
#if UE_COMPUTE_PLATFORM_MAC
	for(;;)
	{
		if (sem_trywait((sem_t*)Handle) == 0)
		{
			return true;
		}
		
		struct timespec currentTs;
		if (clock_gettime(CLOCK_REALTIME, &currentTs) != 0 || currentTs.tv_sec > ts.tv_sec || (currentTs.tv_sec == ts.tv_sec && currentTs.tv_nsec > ts.tv_nsec))
		{
			return false;
		}
		
		struct timespec sleepTs = { 0, };
		sleepTs.tv_nsec = 100 * NsPerMs;
		
		nanosleep(&sleepTs, nullptr);
	}
#else
	return sem_timedwait((sem_t*)Handle, &ts) == 0;
#endif
#endif
}

/////////////////////////////////////////////////// 

FComputeMemoryMappedFile::FComputeMemoryMappedFile()
	: Handle(nullptr)
	, Pointer(nullptr)
	, MappedSize(0)
	, OwnerName(nullptr)
	, Anonymous(false)
{
}

FComputeMemoryMappedFile::~FComputeMemoryMappedFile()
{
	Close();
}

bool FComputeMemoryMappedFile::Create(const char* Name, long long Capacity)
{
	Close();

	Anonymous = Name == nullptr;

	if (Anonymous)
	{
		Pointer = FMemory::Malloc(Capacity);
		MappedSize = Capacity;
		return true;
	}

#if UE_COMPUTE_PLATFORM_WINDOWS
	LARGE_INTEGER LargeInteger;
	LargeInteger.QuadPart = Capacity;

	Handle = CreateFileMappingA(INVALID_HANDLE_VALUE, NULL, PAGE_READWRITE, LargeInteger.HighPart, LargeInteger.LowPart, Name);
	if (Handle == nullptr)
	{
		return false;
	}

	Pointer = MapViewOfFile(Handle, FILE_MAP_ALL_ACCESS, 0, 0, 0);
	if (Pointer == nullptr)
	{
		return false;
	}

	return true;
#else
	int Fd = shm_open(Name, O_CREAT | O_EXCL | O_RDWR, 0666);
	if(Fd < 0)
	{
		std::cerr << "Unable to create shared memory object '" << Name << "' " << FFormatErrno() << std::endl;
		return false;
	}

	Handle = (void*)(size_t)Fd;
	MappedSize = Capacity;
	OwnerName = strdup(Name);

	if(ftruncate(Fd, MappedSize) < 0)
	{
		std::cerr << "Unable to update size of shared memory object '" << Name << "' to " << MappedSize << " " << FFormatErrno() << std::endl;
		return false;
	}

	Pointer = mmap(nullptr, MappedSize, PROT_READ | PROT_WRITE, MAP_SHARED, Fd, 0);
	if(Pointer == MAP_FAILED)
	{
		std::cerr << "Unable to map shared memory object '" << Name << " " << FFormatErrno() << std::endl;
		return false;
	}
	
	return true;
#endif
}

bool FComputeMemoryMappedFile::OpenExisting(const char* Name)
{
	Close();

	check(Name);
	Anonymous = false;

#if UE_COMPUTE_PLATFORM_WINDOWS
	Handle = OpenFileMappingA(FILE_MAP_ALL_ACCESS, true, Name);
	if (Handle == nullptr)
	{
		return false;
	}

	Pointer = MapViewOfFile(Handle, FILE_MAP_ALL_ACCESS, 0, 0, 0);
	if (Pointer == nullptr)
	{
		return false;
	}

	return true;
#else
	int Fd = shm_open(Name, O_RDWR, 0666);
	if(Fd < 0)
	{
		std::cerr << "Unable to open shared memory object '" << Name << "' " << FFormatErrno() << std::endl;
		return false;
	}

	Handle = (void*)(size_t)Fd;

	struct stat st;
	fstat(Fd, &st);
	MappedSize = st.st_size;

	Pointer = mmap(nullptr, MappedSize, PROT_READ | PROT_WRITE, MAP_SHARED, Fd, 0);
	if(Pointer == MAP_FAILED)
	{
		std::cerr << "Unable to map shared memory object '" << Name << " " << FFormatErrno() << std::endl;
		return false;
	}
	
	return true;
#endif
}

void FComputeMemoryMappedFile::Close()
{
	if (Anonymous)
	{
		FMemory::Free(Pointer);
		Pointer = nullptr;
		return;
	}

#if UE_COMPUTE_PLATFORM_WINDOWS
	if (Pointer != nullptr)
	{
		UnmapViewOfFile(Pointer);
		Pointer = nullptr;
	}

	if (Handle != nullptr)
	{
		CloseHandle(Handle);
		Handle = nullptr;
	}
#else
	if(OwnerName != nullptr)
	{
		shm_unlink(OwnerName);
		free(OwnerName);
		OwnerName = nullptr;
	}
	
	if(Pointer != nullptr)
	{
		munmap(Pointer, MappedSize);
		Pointer = nullptr;
	}

	int Fd = (int)(size_t)Handle;
	if(Fd > 0)
	{
		close(Fd);
		Handle = nullptr;
	}
#endif
}

void* FComputeMemoryMappedFile::GetPointer() const
{
#if UE_COMPUTE_PLATFORM_WINDOWS
	return Pointer;
#else
	return (unsigned char*)Pointer + 16;
#endif
}
