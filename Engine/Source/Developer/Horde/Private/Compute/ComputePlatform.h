// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include <stdio.h>
#include <stddef.h>
#include <stdint.h>

//
// IPC manually-reset event object
//
class FComputeEvent
{
public:
	FComputeEvent();
	~FComputeEvent();

	// Creates a new event with the given name
	HORDE_API bool Create(const char* Name);

	// Opens an existing event created elsewhere
	HORDE_API bool OpenExisting(const char* Name);

	// Close the event and release its resources
	HORDE_API void Close();

	// Signal the event, releasing any waiters
	HORDE_API void Signal();

	// Wait for the event to be signalled or timeout. Pass -1 for timeoutMs to wait infinitely.
	HORDE_API bool Wait(int timeoutMs);

private:
	void* Handle;
	bool Anonymous;
};

//
// IPC memory mapped file
//
class FComputeMemoryMappedFile
{
public:
	HORDE_API FComputeMemoryMappedFile();
	HORDE_API ~FComputeMemoryMappedFile();

	// Creates a new memory mapped file with the given capacity
	HORDE_API bool Create(const char* Name, long long Capacity);

	// Opens an existing memory mapped file
	HORDE_API bool OpenExisting(const char* Name);

	// Close the memory mapped file handle
	HORDE_API void Close();

	// Gets a pointer to the mapped memory
	HORDE_API void* GetPointer() const;

private:
	void* Handle;
	void* Pointer;
	long long MappedSize;
	char* OwnerName;
	bool Anonymous;
};
