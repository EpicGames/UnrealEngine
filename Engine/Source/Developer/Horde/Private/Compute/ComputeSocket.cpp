// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/ComputeSocket.h"
#include "Compute/ComputePlatform.h"
#include "HAL/CriticalSection.h"
#include "HAL/Event.h"
#include "Misc/ScopeLock.h"
#include <iostream>
#include <assert.h>
#include <unordered_set>
#include <unordered_map>
#include <vector>
#include <thread>
#include <chrono>
#include "../HordePlatform.h"

FComputeSocket::FComputeSocket()
{
}

FComputeSocket::~FComputeSocket()
{
}

TSharedPtr<FComputeChannel> FComputeSocket::CreateChannel(int ChannelId, bool Anonymous)
{
	FComputeBuffer::FParams Params;
	Params.Anonymous = Anonymous;

	FComputeBuffer RecvBuffer;
	if (!RecvBuffer.CreateNew(Params))
	{
		return {};
	}

	FComputeBuffer SendBuffer;
	if (!SendBuffer.CreateNew(Params))
	{
		return {};
	}

	return CreateChannel(ChannelId, std::move(RecvBuffer), std::move(SendBuffer));
}

TSharedPtr<FComputeChannel> FComputeSocket::CreateChannel(int ChannelId, FComputeBuffer RecvBuffer, FComputeBuffer SendBuffer)
{
	TSharedPtr<FComputeChannel> Channel = MakeShared<FComputeChannel>(RecvBuffer.CreateReader(), SendBuffer.CreateWriter());

	AttachRecvBuffer(ChannelId, std::move(RecvBuffer));
	AttachSendBuffer(ChannelId, std::move(SendBuffer));

	return Channel;
}

//////////////////////////////////////////////////////

const char* const FWorkerComputeSocket::IpcEnvVar = "UE_HORDE_COMPUTE_IPC";

enum class FWorkerComputeSocket::EMessageType
{
	AttachRecvBuffer = 0,
	AttachSendBuffer = 1,
};

FWorkerComputeSocket::FWorkerComputeSocket()
{
}

FWorkerComputeSocket::~FWorkerComputeSocket()
{
	Close();
}

void FWorkerComputeSocket::StartCommunication()
{
}

bool FWorkerComputeSocket::Open()
{
	char EnvVar[FComputeBuffer::MaxNameLength];
	if (!FHordePlatform::GetEnvironmentVariable(IpcEnvVar, EnvVar, sizeof(EnvVar) / sizeof(EnvVar[0])))
	{
		return false;
	}

	return Open(EnvVar);
}

bool FWorkerComputeSocket::Open(const char* CommandBufferName)
{
	FComputeBuffer CommandBuffer;
	if (CommandBuffer.OpenExisting(CommandBufferName))
	{
		CommandBufferWriter = CommandBuffer.CreateWriter();
		return true;
	}
	return false;
}

void FWorkerComputeSocket::Close()
{
	CommandBufferWriter.Close();
}

void FWorkerComputeSocket::AttachRecvBuffer(int ChannelId, FComputeBuffer RecvBuffer)
{
	AttachBuffer(ChannelId, EMessageType::AttachRecvBuffer, RecvBuffer.GetName());
	Buffers.push_back(std::move(RecvBuffer));
}

void FWorkerComputeSocket::AttachSendBuffer(int ChannelId, FComputeBuffer SendBuffer)
{
	AttachBuffer(ChannelId, EMessageType::AttachSendBuffer, SendBuffer.GetName());
	Buffers.push_back(std::move(SendBuffer));
}

void FWorkerComputeSocket::AttachBuffer(int ChannelId, EMessageType Type, const char* Name)
{
	unsigned char* Data = CommandBufferWriter.WaitToWrite(1024);

	size_t Len = 0;
	Len += WriteVarUInt(Data + Len, (unsigned char)Type);
	Len += WriteVarUInt(Data + Len, (unsigned int)ChannelId);
	Len += WriteString(Data + Len, Name);

	CommandBufferWriter.AdvanceWritePosition(Len);
}

void FWorkerComputeSocket::RunServer(FComputeBufferReader& CommandBufferReader, FComputeSocket& Socket)
{
	const unsigned char* Message;
	while ((Message = CommandBufferReader.WaitToRead(1)) != nullptr)
	{
		size_t Len = 0;

		unsigned int Type;
		Len += ReadVarUInt(Message + Len, &Type);

		EMessageType MessageType = (EMessageType)*Message;
		switch (MessageType)
		{
		case EMessageType::AttachSendBuffer:
			{
				unsigned int ChannelId;
				Len += ReadVarUInt(Message + Len, &ChannelId);

				char Name[FComputeBuffer::MaxNameLength];
				Len += ReadString(Message + Len, Name, FComputeBuffer::MaxNameLength);

				FComputeBuffer Buffer;
				if (Buffer.OpenExisting(Name))
				{
					Socket.AttachSendBuffer(ChannelId, Buffer);
				}
				else
				{
					check(false);
				}
			}
			break;
		case EMessageType::AttachRecvBuffer:
			{
				unsigned int ChannelId;
				Len += ReadVarUInt(Message + Len, &ChannelId);

				char Name[FComputeBuffer::MaxNameLength];
				Len += ReadString(Message + Len, Name, FComputeBuffer::MaxNameLength);

				FComputeBuffer Buffer;
				if (Buffer.OpenExisting(Name))
				{
					Socket.AttachRecvBuffer(ChannelId, Buffer);
				}
				else
				{
					check(false);
				}
			}
			break;
		default:
			check(false);
			return;
		}

		CommandBufferReader.AdvanceReadPosition(Len);
	}
}

size_t FWorkerComputeSocket::ReadVarUInt(const unsigned char* Pos, unsigned int* OutValue)
{
	size_t ByteCount = FHordePlatform::CountLeadingZeros((unsigned char)(~*static_cast<const unsigned char*>(Pos))) - 23;

	unsigned int Value = *Pos++ & (unsigned char)(0xff >> ByteCount);
	switch (ByteCount - 1)
	{
	case 8: Value <<= 8; Value |= *Pos++;
	case 7: Value <<= 8; Value |= *Pos++;
	case 6: Value <<= 8; Value |= *Pos++;
	case 5: Value <<= 8; Value |= *Pos++;
	case 4: Value <<= 8; Value |= *Pos++;
	case 3: Value <<= 8; Value |= *Pos++;
	case 2: Value <<= 8; Value |= *Pos++;
	case 1: Value <<= 8; Value |= *Pos++;
	default:
		break;
	}

	*OutValue = Value;
	return ByteCount;
}

size_t FWorkerComputeSocket::ReadString(const unsigned char* Pos, char* OutText, size_t OutTextMaxLen)
{
	unsigned int TextLen;

	size_t Len = ReadVarUInt(Pos, &TextLen);
	FCStringAnsi::Strncpy(OutText, (const char*)Pos + Len, OutTextMaxLen);

	return Len + TextLen;
}

size_t FWorkerComputeSocket::WriteVarUInt(unsigned char* Pos, unsigned int Value)
{
	// Use BSR to return the log2 of the integer
	// return 0 if value is 0
	unsigned int ByteCount = (unsigned int)(int(FHordePlatform::FloorLog2(Value)) / 7 + 1);

	unsigned char* OutBytes = Pos + ByteCount - 1;
	switch (ByteCount - 1)
	{
	case 4: *OutBytes-- = (unsigned char)(Value); Value >>= 8; [[fallthrough]];
	case 3: *OutBytes-- = (unsigned char)(Value); Value >>= 8; [[fallthrough]];
	case 2: *OutBytes-- = (unsigned char)(Value); Value >>= 8; [[fallthrough]];
	case 1: *OutBytes-- = (unsigned char)(Value); Value >>= 8; [[fallthrough]];
	default: 
		break;
	}
	*OutBytes = (unsigned char)(0xff << (9 - ByteCount)) | (unsigned char)(Value);

	return ByteCount;
}

size_t FWorkerComputeSocket::WriteString(unsigned char* Pos, const char* Text)
{
	size_t TextLen = strlen(Text);

	size_t Len = WriteVarUInt(Pos, (int)TextLen);
	memcpy((char*)Pos + Len, Text, TextLen);

	return Len + TextLen;
}

//////////////////////////////////////////////////////

class FRemoteComputeSocket : public FComputeSocket
{
public:
	enum class EControlMessageType
	{
		Detach = -2,
	};

	struct FFrameHeader
	{
		int32 Channel;
		int32 Size;
	};

	TUniquePtr<FComputeTransport> Transport;
	const EComputeSocketEndpoint Endpoint;
	FCriticalSection CriticalSection;

	FEventRef PingThreadFinishCV;
	std::thread PingThread;

	std::thread RecvThread;

	std::unordered_map<int, FComputeBufferWriter> Writers;
	std::vector<FComputeBufferReader> Readers;
	std::unordered_map<int, std::thread> SendThreads;

	FRemoteComputeSocket(TUniquePtr<FComputeTransport> InTransport, EComputeSocketEndpoint InEndpoint)
		: Transport(MoveTemp(InTransport))
		, Endpoint(InEndpoint)
		, PingThreadFinishCV(EEventMode::ManualReset)
	{
	}

	~FRemoteComputeSocket() override
	{
		PingThreadFinishCV->Trigger();

		for (FComputeBufferReader& Reader : Readers)
		{
			Reader.Detach();
		}

		for (std::pair<const int, std::thread>& Pair : SendThreads)
		{
			Pair.second.join();
		}

		Transport->Close();

		// Only join receive and ping threads if they started execution yet
		if (RecvThread.joinable())
		{
			check(PingThread.joinable());
			RecvThread.join();
			PingThread.join();
		}
	}

	virtual void StartCommunication()
	{
		// Initialize the receiver thread after having attached channel 0
		RecvThread = std::thread(&FRemoteComputeSocket::RecvThreadProc, this);
		PingThread = std::thread(&FRemoteComputeSocket::PingThreadProc, this);
	}

	void PingThreadProc()
	{
		for (;;)
		{
			{ // Send the ping message
				FScopeLock Lock(&CriticalSection);

				FFrameHeader Header;
				Header.Channel = 0;
				Header.Size = -3; // Ping control message.

				Transport->SendMessage(&Header, sizeof(Header));
			}

			if (PingThreadFinishCV->Wait(2000))
			{
				break;
			}
		}
	}

	void RecvThreadProc()
	{
		std::unordered_map<int, FComputeBufferWriter> CachedWriters;

		// Process messages from the remote
		FFrameHeader Header;
		while (Transport->RecvMessage(&Header, sizeof(Header)))
		{
			if (Header.Size >= 0)
			{
				if (!ReadFrame(CachedWriters, Header.Channel, Header.Size))
				{
					UE_LOGF(LogHorde, Log, "Failed to read frame header (Channel %d, Size %d)", Header.Channel, Header.Size);
					return;
				}
			}
			else if (Header.Size == (int)EControlMessageType::Detach)
			{
				DetachRecvBuffer(CachedWriters, Header.Channel);
			}
			else
			{
				UE_LOGF(LogHorde, Warning, "Invalid frame header size received (%d)", Header.Size);
				return;
			}
		}
	}

	void SendThreadProc(int Channel, FComputeBufferReader Reader)
	{
		FFrameHeader Header;
		Header.Channel = Channel;

		const unsigned char* Data;
		while ((Data = Reader.WaitToRead(1)) != nullptr)
		{
			FScopeLock Lock(&CriticalSection);
			Header.Size = (int)Reader.GetMaxReadSize();
			Transport->SendMessage(&Header, sizeof(Header));
			Transport->SendMessage(Data, Header.Size);
			Reader.AdvanceReadPosition(Header.Size);
		}

		if (Reader.IsComplete())
		{
			FScopeLock Lock(&CriticalSection);
			Header.Size = (int)EControlMessageType::Detach;
			Transport->SendMessage(&Header, sizeof(Header));
		}
	}

	bool ReadFrame(std::unordered_map<int, FComputeBufferWriter>& CachedWriters, int Channel, int Size)
	{
		std::unordered_map<int, FComputeBufferWriter>::iterator Iter = CachedWriters.find(Channel);
		if (Iter == CachedWriters.end())
		{
			FScopeLock Lock(&CriticalSection);

			Iter = Writers.find(Channel);
			if (Iter == Writers.end())
			{
				return false;
			}

			Iter = CachedWriters.insert(*Iter).first;
		}

		FComputeBufferWriter& Writer = Iter->second;

		unsigned char* Data = Writer.WaitToWrite(Size);
		if (!Transport->RecvMessage(Data, Size))
		{
			return false;
		}

		Writer.AdvanceWritePosition(Size);
		return true;
	}

	void AttachRecvBuffer(int ChannelId, FComputeBuffer RecvBuffer) override
	{
		FScopeLock Lock(&CriticalSection);
		FComputeBufferWriter Writer = RecvBuffer.CreateWriter();
		Writers.insert(std::pair<int, FComputeBufferWriter>(ChannelId, std::move(Writer)));
	}

	void AttachSendBuffer(int ChannelId, FComputeBuffer SendBuffer) override
	{
		FScopeLock Lock(&CriticalSection);
		FComputeBufferReader Reader = SendBuffer.CreateReader();
		Readers.push_back(Reader);
		SendThreads.insert(std::make_pair(ChannelId, std::thread(&FRemoteComputeSocket::SendThreadProc, this, ChannelId, std::move(Reader))));
	}

	void DetachRecvBuffer(std::unordered_map<int, FComputeBufferWriter>& CachedWriters, int Channel)
	{
		CachedWriters.erase(Channel);

		FScopeLock Lock(&CriticalSection);

		std::unordered_map<int, FComputeBufferWriter>::iterator Iter = Writers.find(Channel);
		if (Iter != Writers.end())
		{
			Iter->second.MarkComplete();
			Writers.erase(Iter);
		}
	}
};

TUniquePtr<FComputeSocket> CreateComputeSocket(TUniquePtr<FComputeTransport> Transport, EComputeSocketEndpoint Endpoint)
{
	return TUniquePtr<FComputeSocket>(new FRemoteComputeSocket(MoveTemp(Transport), Endpoint));
}
