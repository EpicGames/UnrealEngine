// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/ComputeTransport.h"
#include <algorithm>

FComputeTransport::~FComputeTransport()
{
}

bool FComputeTransport::SendMessage(const void* Data, size_t Size)
{
	const unsigned char* RemainingData = (const unsigned char*)Data;
	for (size_t RemainingSize = Size; RemainingSize > 0; )
	{
		size_t SentSize = Send(RemainingData, RemainingSize);
		if (SentSize == 0)
		{
			return false;
		}

		RemainingData += SentSize;
		RemainingSize -= SentSize;
	}
	return true;
}

bool FComputeTransport::RecvMessage(void* Data, size_t Size)
{
	unsigned char* RemainingData = (unsigned char*)Data;
	for (size_t RemainingSize = Size; RemainingSize > 0; )
	{
		size_t RecvSize = Recv(RemainingData, RemainingSize);
		if (RecvSize == 0)
		{
			return false;
		}

		RemainingData += RecvSize;
		RemainingSize -= RecvSize;
	}
	return true;
}

///////////////////////////////////

FBufferTransport::FBufferTransport(FComputeBufferWriter InSendBufferWriter, FComputeBufferReader InRecvBufferReader)
	: SendBufferWriter(std::move(InSendBufferWriter))
	, RecvBufferReader(std::move(InRecvBufferReader))
{
}

bool FBufferTransport::IsValid() const
{
	return SendBufferWriter.IsValid() && RecvBufferReader.IsValid();
}

size_t FBufferTransport::Send(const void* Data, size_t Size)
{
	unsigned char* Buffer = SendBufferWriter.WaitToWrite(1);

	size_t WriteSize = std::min(Size, SendBufferWriter.GetMaxWriteSize());
	memcpy(Buffer, Data, WriteSize);
	SendBufferWriter.AdvanceWritePosition(WriteSize);

	return WriteSize;
}

size_t FBufferTransport::Recv(void* Data, size_t Size)
{
	const unsigned char* Buffer = RecvBufferReader.WaitToRead(1);

	size_t ReadSize = std::min(Size, RecvBufferReader.GetMaxReadSize());
	memcpy(Data, Buffer, ReadSize);
	RecvBufferReader.AdvanceReadPosition(ReadSize);

	return ReadSize;
}

void FBufferTransport::MarkComplete()
{
	SendBufferWriter.MarkComplete();
}

void FBufferTransport::Close()
{
	RecvBufferReader.Detach();
}