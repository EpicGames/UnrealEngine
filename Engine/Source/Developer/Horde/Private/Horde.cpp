// Copyright Epic Games, Inc. All Rights Reserved.

#include "Horde.h"

#include "DesktopPlatformModule.h"
#include "HordePlatform.h"
#include "IDesktopPlatform.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"


DEFINE_LOG_CATEGORY(LogHorde);

struct FHordeModule : IModuleInterface
{
	virtual void StartupModule() override
	{
	}
};

IMPLEMENT_MODULE(FHordeModule, Horde)

// --------------------------------------------------------------------------------

bool FHorde::GetServerUrl(FString& OutUrl, FString* OutUrlConfigSource)
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::TryGet();
	if (DesktopPlatform == nullptr)
	{
		return false;
	}
	return DesktopPlatform->GetHordeUrl(OutUrl, OutUrlConfigSource);
}

FString FHorde::GetTemplateName()
{
	static FString TemplateName;

	if (TemplateName.IsEmpty())
	{
		if (false == FParse::Value(FCommandLine::Get(), TEXT("HordeTemplateName="), TemplateName))
		{
			TemplateName = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_HORDE_TEMPLATENAME"));
		}
	}
	return TemplateName;
}

FString FHorde::GetTemplateId()
{
	static FString TemplateId;

	if (TemplateId.IsEmpty())
	{
		if (false == FParse::Value(FCommandLine::Get(), TEXT("HordeTemplateId="), TemplateId))
		{
			TemplateId = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_HORDE_TEMPLATEID"));
		}
	}

	return TemplateId;
}

FString FHorde::GetJobId()
{
	static FString JobId;

	if (JobId.IsEmpty())
	{
		if (false == FParse::Value(FCommandLine::Get(), TEXT("HordeJobId="), JobId))
		{
			JobId = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_HORDE_JOBID"));
		}
	}

	return JobId;
}

FString FHorde::GetJobURL()
{
	struct FJobUrlInitializer
	{
		FString Value;

		FJobUrlInitializer()
		{
			if (!FParse::Value(FCommandLine::Get(), TEXT("HordeJobUrl="), Value))
			{
				Value = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_HORDE_URL"));
				if (Value.Len() > 0)
				{
					Value /= FString::Printf(TEXT("job/%s"), *GetJobId());
				}
			}
		}
	};

	static const FJobUrlInitializer JobUrl;
	return JobUrl.Value;
}

FString FHorde::GetStepId()
{
	static FString StepId;

	if (StepId.IsEmpty())
	{
		if (false == FParse::Value(FCommandLine::Get(), TEXT("HordeStepId="), StepId))
		{
			StepId = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_HORDE_STEPID"));
		}
	}
	return StepId;
}

FString FHorde::GetStepURL()
{
	return FString::Printf(TEXT("%s?step=%s"), *GetJobURL(), *GetStepId());
}

FString FHorde::GetStepName()
{
	FString StepName;

	if (StepName.IsEmpty())
	{
		if (false == FParse::Value(FCommandLine::Get(), TEXT("HordeStepName="), StepName))
		{
			StepName = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_HORDE_STEPNAME"));
		}
	}
	return StepName;
}


FString FHorde::GetBatchId()
{
	FString BatchId;
	if (BatchId.IsEmpty())
	{
		if (false == FParse::Value(FCommandLine::Get(), TEXT("HordeBatchId="), BatchId))
		{
			BatchId = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_HORDE_BATCHID"));
		}
	}
	return BatchId;
} 

FString FHorde::GetServerURL()
{
	static FString ServerURL;

	if (ServerURL.IsEmpty())
	{
		if (false == FParse::Value(FCommandLine::Get(), TEXT("HordeServerUrl="), ServerURL))
		{
			ServerURL = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_HORDE_URL"));
		}
	}
	return ServerURL;
}