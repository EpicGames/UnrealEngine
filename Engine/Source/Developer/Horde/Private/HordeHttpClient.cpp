// Copyright Epic Games, Inc. All Rights Reserved.

#include "HordeHttpClient.h"
#include "Server/ServerMessages.h"
#include "Interfaces/IHttpResponse.h"
#include "DesktopPlatformModule.h"
#include "IDesktopPlatform.h"
#include "HttpModule.h"
#include "Misc/Paths.h"
#include "HAL/PlatformMisc.h"

FHordeHttpClient::FHordeHttpClient(FString InServerUrl)
	: ServerUrl(InServerUrl)
{
}

FHordeHttpClient::~FHordeHttpClient()
{
}

bool FHordeHttpClient::Login(bool bUnattended, FFeedbackContext* Warn)
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::TryGet();
	if (DesktopPlatform != nullptr)
	{
		FString NewToken;
		FDateTime ExpiresAt;
		bool bWasInteractive = false;

		EDesktopLoginInteractionLevel InteractionLevel = bUnattended ? EDesktopLoginInteractionLevel::TimeLimited : EDesktopLoginInteractionLevel::Interactive;
		if (DesktopPlatform->GetHordeAccessToken(ServerUrl, InteractionLevel, Warn, NewToken, ExpiresAt, bWasInteractive))
		{
			Token = NewToken;
			return true;
		}
	}
	return false;
}

bool FHordeHttpClient::LoginWithOidc(const TCHAR* Profile, bool bUnattended, FFeedbackContext* Warn)
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::TryGet();
	if (DesktopPlatform != nullptr)
	{
		FString NewToken;
		FDateTime ExpiresAt;
		bool bWasInteractive = false;

		EDesktopLoginInteractionLevel InteractionLevel = bUnattended ? EDesktopLoginInteractionLevel::TimeLimited : EDesktopLoginInteractionLevel::Interactive;
		if (DesktopPlatform->GetOidcAccessToken(FPaths::RootDir(), FString(), Profile, InteractionLevel, Warn, NewToken, ExpiresAt, bWasInteractive))
		{
			Token = NewToken;
			return true;
		}
	}
	return false;
}

bool FHordeHttpClient::LoginWithEnvironmentVariable()
{
	FString EnvVarHordeToken = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_HORDE_TOKEN"));
	if (!EnvVarHordeToken.IsEmpty())
	{
		Token = MoveTemp(EnvVarHordeToken);
		return true;
	}
	return false;
}

TSharedRef<IHttpRequest> FHordeHttpClient::CreateRequest(const TCHAR* Verb, const TCHAR* Path)
{
	FHttpModule& Module = FHttpModule::Get();

	TSharedRef<IHttpRequest> Request = Module.CreateRequest();
	Request->SetVerb(Verb);
	Request->SetURL(ServerUrl / Path);
	if (Token.Len() > 0)
	{
		Request->SetHeader(TEXT("Authorization"), FString::Printf(TEXT("Bearer %s"), *Token));
	}

	return Request;
}

TSharedRef<IHttpResponse> FHordeHttpClient::Get(const TCHAR* Path)
{
	return ExecuteRequest(CreateRequest(TEXT("GET"), Path));
}

TSharedRef<IHttpResponse> FHordeHttpClient::ExecuteRequest(TSharedRef<IHttpRequest> Request)
{
	verify(Request->ProcessRequest());

	for (;;)
	{
		FHttpResponsePtr Response = Request->GetResponse();
		if (Response)
		{
			return Response.ToSharedRef();
		}
		FPlatformProcess::Sleep(0.05f);
	}
}
