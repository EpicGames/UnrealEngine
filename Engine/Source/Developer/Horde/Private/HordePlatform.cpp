// Copyright Epic Games, Inc. All Rights Reserved.

#include "HordePlatform.h"
#include <assert.h>
#include <wchar.h>
#include <bit>
#include <algorithm>
#include <iostream>

// Defines for the current platform
#ifdef _MSC_VER
#define UE_HORDE_PLATFORM_WINDOWS 1
#define UE_HORDE_PLATFORM_MAC 0
#define UE_HORDE_PLATFORM_LINUX 0
#elif defined(__APPLE__)
#define UE_HORDE_PLATFORM_WINDOWS 0
#define UE_HORDE_PLATFORM_MAC 1
#define UE_HORDE_PLATFORM_LINUX 0
#else
#define UE_HORDE_PLATFORM_WINDOWS 0
#define UE_HORDE_PLATFORM_MAC 0
#define UE_HORDE_PLATFORM_LINUX 1
#endif

#if UE_HORDE_PLATFORM_WINDOWS
#include <Windows.h>
#undef min
#undef max
#undef GetEnvironmentVariable
#undef SendMessage
#undef InterlockedIncrement
#else
#include <semaphore.h>
#include <unistd.h>
#include <atomic>
#include <sys/mman.h>
#include <sys/stat.h>
#include <time.h>
#include <fcntl.h>
#endif

void FHordePlatform::NotImplemented()
{
#if PLATFORM_EXCEPTIONS_DISABLED
	abort();
#else
	throw std::string("Not Implemented");
#endif
}

void FHordePlatform::NotSupported(const char* Message)
{
#if PLATFORM_EXCEPTIONS_DISABLED
	abort();
#else
	throw std::string(Message);
#endif
}

bool FHordePlatform::GetEnvironmentVariable(const char* Name, char* Buffer, size_t BufferLen)
{
#if PLATFORM_WINDOWS
	int Length = GetEnvironmentVariableA(Name, Buffer, (DWORD)BufferLen);
	return Length > 0 && Length < BufferLen;
#else
	char* Value = getenv(Name);
	if (Value != nullptr)
	{
		FCStringAnsi::Strncpy(Buffer, Value, BufferLen);
		return true;
	}
	return false;
#endif
}

void FHordePlatform::CreateUniqueIdentifier(char* NameBuffer, size_t NameBufferLen)
{
	static int32 Counter = 0;
	uint32 Index = (uint32)FPlatformAtomics::InterlockedIncrement(&Counter);

#if UE_HORDE_PLATFORM_WINDOWS
	uint32 Pid = GetCurrentProcessId();
	uint64 Time = GetTickCount64();
#else
	struct timespec ts;
	clock_gettime(CLOCK_MONOTONIC, &ts);
	uint32 Pid = (uint32)getpid();
	uint64 Time = (uint64)(ts.tv_sec%100000);
#endif

	TExternalStringBuilder<char> Builder(NameBuffer, NameBufferLen);

	// Encode to be as short as possible.. 
	auto AppendValue = [&](auto Value)
	{
		constexpr char CharSet[] = "0123456789ABCDEFGHIJKLMNOPQRSTUVWXYZabcdefghijklmnopqrstuvwxyz";
		constexpr int Base = sizeof(CharSet) - 1;
		do
		{
			Builder.AppendChar(CharSet[Value % Base]);
			Value /= Base;
		}
		while (Value);
	};

	AppendValue(Pid);
	Builder.AppendChar('_');
	AppendValue(Time);
	Builder.AppendChar('_');
	AppendValue(Index);
	Builder.ToString(); // To ensure terminated
}

void FHordePlatform::CreateUniqueName(char* NameBuffer, size_t NameBufferLen)
{
#if UE_HORDE_PLATFORM_WINDOWS
	const char Prefix[] = "Local\\COMPUTE_";
#else
	const char Prefix[] = "/UEC_";
#endif

	size_t PrefixLen = (sizeof(Prefix) / sizeof(Prefix[0])) - 1;
	check(NameBufferLen > PrefixLen);

	memcpy(NameBuffer, Prefix, PrefixLen * sizeof(char));
	CreateUniqueIdentifier(NameBuffer + PrefixLen, NameBufferLen - PrefixLen);
}

unsigned int FHordePlatform::FloorLog2(unsigned int Value)
{
	return std::max<unsigned int>(0, 31 - std::countl_zero(Value));
}

unsigned int FHordePlatform::CountLeadingZeros(unsigned int Value)
{
	return std::countl_zero(Value);
}

bool FHordePlatform::TryParseSizeT(const char* Source, size_t SourceLen, size_t& OutValue, size_t& OutNumBytes)
{
	OutValue = 0;
	OutNumBytes = 0;

	for (; OutNumBytes < SourceLen; OutNumBytes++)
	{
		char Character = Source[OutNumBytes];
		if (Character < '0' || Character > '9')
		{
			break;
		}
		OutValue = (OutValue * 10) + (Character - '0');
	}

	return OutNumBytes > 0;
}
