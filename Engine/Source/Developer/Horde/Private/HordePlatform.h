// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

DECLARE_LOG_CATEGORY_EXTERN(LogHorde, Log, All);

//
// Platform-specific utility functions
//
struct FHordePlatform
{
	//
	// General
	//

	// Indicate that the current codepath is not implemented
	[[noreturn]] static void NotImplemented();

	// Indicate that the current codepath is not supported
	[[noreturn]] static void NotSupported(const char* Message);

	// Reads an environment variable
	static bool GetEnvironmentVariable(const char* Name, char* Buffer, size_t BufferLen);

	// Creates a unique identifier
	static void CreateUniqueIdentifier(char* NameBuffer, size_t NameBufferLen);

	// Creates a unique object name
	static void CreateUniqueName(char* NameBuffer, size_t NameBufferLen);

	//
	// Math
	//

	// Find the log2 of the given value, returning 0 if the value is zero.
	static unsigned int FloorLog2(unsigned int Value);

	// Count the number of leading zeros in the given value.
	static unsigned int CountLeadingZeros(unsigned int Value);

	//
	// Strings
	//

	// Parse an integer from a string, returning the number of bytes read
	static bool TryParseSizeT(const char* Source, size_t SourceLen, size_t& OutValue, size_t& OutNumBytes);
};
