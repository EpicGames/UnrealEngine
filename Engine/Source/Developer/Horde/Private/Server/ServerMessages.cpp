// Copyright Epic Games, Inc. All Rights Reserved.

#include "Server/ServerMessages.h"

FGetServerInfoResponse::FGetServerInfoResponse()
{
}

FGetServerInfoResponse::~FGetServerInfoResponse()
{
}

void FGetServerInfoResponse::Serialize(FJsonSerializerBase& Serializer, bool bFlatObject)
{
	JSON_SERIALIZE("ServerVersion", ServerVersion);
	JSON_SERIALIZE("AgentVersion", AgentVersion);
	JSON_SERIALIZE("OsDescription", OsDescription);
}

// -------------------------------------

FGetAuthConfigResponse::FGetAuthConfigResponse()
{
}

FGetAuthConfigResponse::~FGetAuthConfigResponse()
{
}

void FGetAuthConfigResponse::Serialize(FJsonSerializerBase& Serializer, bool bFlatObject)
{
	JSON_SERIALIZE("Method", Method);
	JSON_SERIALIZE("ServerUrl", ServerUrl);
	JSON_SERIALIZE("ClientId", ClientId);
	JSON_SERIALIZE_ARRAY("LocalRedirectUrls", LocalRedirectUrls);
}

