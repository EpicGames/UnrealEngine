// Copyright Epic Games, Inc. All Rights Reserved.

#include "SharedBufferView.h"

FSharedBufferView::FSharedBufferView()
{
}

FSharedBufferView::FSharedBufferView(FSharedBuffer InBuffer)
	: Buffer(MoveTemp(InBuffer))
	, View(Buffer.GetView())
{
}

FSharedBufferView::FSharedBufferView(FSharedBuffer InBuffer, const FMemoryView& InView)
	: Buffer(MoveTemp(InBuffer))
	, View(InView)
{
}

FSharedBufferView::FSharedBufferView(FSharedBuffer InBuffer, size_t InOffset, size_t InLength)
	: Buffer(MoveTemp(InBuffer))
	, View(Buffer.GetView().Mid(InOffset, InLength))
{
}

FSharedBufferView::~FSharedBufferView()
{
}

FSharedBufferView FSharedBufferView::Copy(const FMemoryView& Span)
{
	return FSharedBufferView(FSharedBuffer::Clone(Span));
}

FSharedBufferView FSharedBufferView::Slice(uint64 Offset) const
{
	return FSharedBufferView(Buffer, View.Mid(Offset));
}

FSharedBufferView FSharedBufferView::Slice(uint64 Offset, uint64 Length) const
{
	return FSharedBufferView(Buffer, View.Mid(Offset, Length));
}

const unsigned char* FSharedBufferView::GetPointer() const
{
	return (const unsigned char*)View.GetData();
}

size_t FSharedBufferView::GetLength() const
{
	return View.GetSize();
}

FMemoryView FSharedBufferView::GetView() const
{
	return View;
}
