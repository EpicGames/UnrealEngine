// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/Blob.h"
#include "Storage/BlobHandle.h"

FBlob::FBlob(const FBlobType& InType, FSharedBufferView InData, TArray<FBlobHandle> InReferences)
	: Type(InType)
	, Data(MoveTemp(InData))
	, References(MoveTemp(InReferences))
{ }

FBlob::~FBlob()
{ }
