// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/BlobHandle.h"
#include "Storage/BlobLocator.h"
#include "Storage/Blob.h"
#include "../HordePlatform.h"

bool FBlobHandle::operator==(const FBlobHandle& Other) const
{
	if (Other.IsValid())
	{
		return Get()->Equals(*Other.Get());
	}
	else
	{
		return !Other.IsValid();
	}
}

bool FBlobHandle::operator!=(const FBlobHandle& Other) const
{
	return !(*this == Other);
}

uint32 GetTypeHash(const FBlobHandle& Handle)
{
	return Handle->GetHashCode();
}

// -----------------------------------------------------------------

FBlobHandleData::~FBlobHandleData()
{
}

bool FBlobHandleData::Equals(const FBlobHandleData& Other) const
{
	return GetType() == Other.GetType();
}

uint32 FBlobHandleData::GetHashCode() const
{
	return (uint32)(size_t)GetType();
}

FBlobType FBlobHandleData::ReadType() const
{
	FBlob Blob = Read();
	return Blob.Type;
}

void FBlobHandleData::ReadImports(TArray<FBlobHandle>& OutImports) const
{
	FBlob Blob = Read();
	OutImports = Blob.References;
}

FSharedBufferView FBlobHandleData::ReadBody() const
{
	FBlob Blob = Read();
	return MoveTemp(Blob.Data);
}

FSharedBufferView FBlobHandleData::ReadBody(size_t Offset, TOptional<size_t> Length) const
{
	FBlob Blob = Read();
	if (Length.IsSet())
	{
		return Blob.Data.Slice(Offset, *Length);
	}
	else
	{
		return MoveTemp(Blob.Data);
	}
}

bool TryAppendLocator(const FBlobHandleData& Handle, FUtf8String& OutPath)
{
	FBlobHandle Outer = Handle.GetOuter();
	if (Outer)
	{
		if (!TryAppendLocator(*Outer, OutPath))
		{
			return false;
		}

		if (!Outer->GetOuter())
		{
			OutPath += '#';
		}
		else
		{
			OutPath += '&';
		}
	}
	return Handle.TryAppendIdentifier(OutPath);
}

FBlobLocator FBlobHandleData::GetLocator() const
{
	FUtf8String Path;
	verify(TryAppendLocator(*this, Path));
	return FBlobLocator(MoveTemp(Path));
}

FBlobHandle FBlobHandleData::GetFragmentHandle(const FUtf8StringView& Fragment) const
{
	FHordePlatform::NotSupported("This handle type does not support nested blobs");
}
