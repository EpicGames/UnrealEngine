// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/BlobLocator.h"
#include "Containers/Utf8String.h"

FBlobLocator::FBlobLocator()
{
}

FBlobLocator::FBlobLocator(FUtf8String InPath)
	: Path(MoveTemp(InPath))
{
}

FBlobLocator::FBlobLocator(const FBlobLocator& InBaseLocator, const FUtf8StringView& InFragment)
{
	if (InFragment.Len() == 0)
	{
		Path = InBaseLocator.Path;
	}
	else
	{
		Path.Reserve(InBaseLocator.Path.Len() + 1 + InFragment.Len());
		Path += InBaseLocator.Path;
		Path += '#';
		Path += InFragment;
	}
}

FBlobLocator::FBlobLocator(const FUtf8StringView& InPath)
	: Path(InPath)
{
}

bool FBlobLocator::IsValid() const
{
	return !Path.IsEmpty();
}

const FUtf8String& FBlobLocator::GetPath() const
{
	return Path;
}

FBlobLocator FBlobLocator::GetBaseLocator() const
{
	int32 HashIdx;
	if (Path.FindChar((UTF8CHAR)'#', HashIdx))
	{
		return FBlobLocator(Path.Mid(0, HashIdx));
	}
	else
	{
		return *this;
	}
}

FUtf8StringView FBlobLocator::GetFragment() const
{
	int32 HashIdx;
	if (Path.FindChar((UTF8CHAR)'#', HashIdx))
	{
		return FUtf8StringView(*Path + HashIdx + 1);
	}
	else
	{
		return FUtf8StringView();
	}
}

bool FBlobLocator::CanUnwrap() const
{
	int32 HashIdx;
	return Path.FindChar((UTF8CHAR)'#', HashIdx);
}

bool FBlobLocator::TryUnwrap(FBlobLocator& OutLocator, FUtf8StringView& OutFragment) const
{
	int32 HashIdx;
	if (Path.FindChar((UTF8CHAR)'#', HashIdx))
	{
		OutLocator = FBlobLocator(Path.Mid(0, HashIdx));
		OutFragment = FUtf8StringView(*Path + HashIdx + 1);
		return true;
	}
	return false;
}

bool FBlobLocator::operator==(const FBlobLocator& Other) const
{
	return Path.Equals(Other.Path, ESearchCase::CaseSensitive);
}

bool FBlobLocator::operator!=(const FBlobLocator& Other) const
{
	return !(*this == Other);
}

uint32 GetTypeHash(const FBlobLocator& Locator)
{
	return GetTypeHash(Locator.Path);
}
