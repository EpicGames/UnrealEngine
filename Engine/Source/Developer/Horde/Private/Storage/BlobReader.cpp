// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/BlobReader.h"
#include "Containers/Utf8String.h"
#include "IO/IoHash.h"
#include "HordePlatform.h"

// ------------------------------------------------------------------------

FBlobReader::FBlobReader(const FBlob& InBlob)
	: FBlobReader(InBlob.Type.Version, InBlob.Data.GetView(), InBlob.References)
{
}

FBlobReader::FBlobReader(int32 InVersion, const FMemoryView& InBuffer, const TArray<FBlobHandle>& InImports)
	: Version(InVersion)
	, Buffer(InBuffer)
	, Imports(InImports)
	, NextImportIdx(0)
{
}

int32 FBlobReader::GetVersion() const
{
	return Version;
}

const unsigned char* FBlobReader::GetBuffer() const
{
	return (const unsigned char*)GetView().GetData();
}

FMemoryView FBlobReader::GetView() const
{
	return Buffer;
}

void FBlobReader::Advance(size_t Size)
{
	Buffer = Buffer.Mid(Size);
}

FBlobHandle FBlobReader::ReadImport()
{
	return Imports[NextImportIdx++];
}

// ------------------------------------------------------------------------

HORDE_API FBlobHandle ReadBlobHandle(FBlobReader& Reader)
{
	return Reader.ReadImport();
}

HORDE_API FBlobHandleWithHash ReadBlobHandleWithHash(FBlobReader& Reader)
{
	FBlobHandle Handle = Reader.ReadImport();
	return FBlobHandleWithHash(MoveTemp(Handle), ReadIoHash(Reader));
}

HORDE_API int ReadInt32(FBlobReader& Reader)
{
	int Value = *(const int*)Reader.GetBuffer();
	Reader.Advance(sizeof(int));
	return Value;
}

HORDE_API FIoHash ReadIoHash(FBlobReader& Reader)
{
	FIoHash Hash;
	memcpy(&Hash, Reader.GetBuffer(), sizeof(FIoHash));
	Reader.Advance(sizeof(FIoHash));
	return Hash;
}

HORDE_API FMemoryView ReadFixedLengthBytes(FBlobReader& Reader, size_t Length)
{
	FMemoryView View = Reader.GetView();
	Reader.Advance(Length);
	return View.Left(Length);
}

HORDE_API size_t ReadUnsignedVarInt(FBlobReader& Reader)
{
	// Figure out the length of the buffer
	const unsigned char* Data = Reader.GetBuffer();
	size_t NumBytes = FHordePlatform::CountLeadingZeros((unsigned char)(~*Data)) - 23;

	// Decode the value
	size_t value = (size_t)(Data[0] & (0xff >> NumBytes));
	for (int i = 1; i < NumBytes; i++)
	{
		value <<= 8;
		value |= Data[i];
	}

	Reader.Advance(NumBytes);
	return value;
}

HORDE_API FUtf8String ReadString(FBlobReader& Reader)
{
	FMemoryView String = ReadStringSpan(Reader);
	return FUtf8String::ConstructFromPtrSize((const UTF8CHAR*)String.GetData(), String.GetSize());
}

HORDE_API FMemoryView ReadStringSpan(FBlobReader& Reader)
{
	size_t Length = ReadUnsignedVarInt(Reader);
	return ReadFixedLengthBytes(Reader, Length);
}

