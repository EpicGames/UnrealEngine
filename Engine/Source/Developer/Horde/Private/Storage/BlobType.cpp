// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/BlobType.h"

const FBlobType FBlobType::Leaf(FGuid(0x1080B643, 0x4A4D8015, 0xFBC2BBAD, 0x818D6C58), 1);

FBlobType::FBlobType(const FGuid& InGuid, int InVersion)
	: Guid(InGuid)
	, Version(InVersion)
{ }

bool FBlobType::operator==(const FBlobType& Other) const
{
	return Guid == Other.Guid && Version == Other.Version;
}

bool FBlobType::operator!=(const FBlobType& Other) const
{
	return !operator==(Other);
}