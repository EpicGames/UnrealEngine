// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/BlobWriter.h"
#include "Storage/RefOptions.h"
#include "IO/IoHash.h"
#include "../HordePlatform.h"

FBlobWriter::~FBlobWriter()
{
}

void FBlobWriter::AddRef(const FRefName& RefName)
{
	AddRef(RefName, FRefOptions());
}

void* FBlobWriter::GetOutputBuffer(size_t Size)
{
	return GetOutputBufferAsSpan(0, Size).GetData();
}

// ------------------------------------------------------------------------------------------

HORDE_API void WriteBlobHandle(FBlobWriter& Writer, FBlobHandle Handle)
{
	Writer.AddImport(MoveTemp(Handle));
}

HORDE_API void WriteBlobHandleWithHash(FBlobWriter& Writer, FBlobHandleWithHash Target)
{
	Writer.AddImport(MoveTemp(Target.Handle));
	WriteIoHash(Writer, Target.Hash);
}

HORDE_API void WriteIoHash(FBlobWriter& Writer, const FIoHash& Hash)
{
	WriteFixedLengthBytes(Writer, &Hash, sizeof(Hash));
}

HORDE_API void WriteFixedLengthBytes(FBlobWriter& Writer, const void* Data, size_t Length)
{
	void* Target = Writer.GetOutputBuffer(Length);
	memcpy(Target, Data, Length);
	Writer.Advance(Length);
}

HORDE_API void WriteFixedLengthBytes(FBlobWriter& Writer, const FMemoryView& View)
{
	WriteFixedLengthBytes(Writer, View.GetData(), View.GetSize());
}

HORDE_API size_t MeasureUnsignedVarInt(size_t Value)
{
	check(Value == (unsigned int)Value);

	if (Value == 0)
	{
		return 1;
	}
	else
	{
		return (FHordePlatform::FloorLog2((unsigned int)Value) / 7) + 1;
	}
}

HORDE_API size_t WriteUnsignedVarInt(void* Buffer, size_t Value)
{
	size_t ByteCount = MeasureUnsignedVarInt(Value);
	WriteUnsignedVarIntWithKnownLength(Buffer, Value, ByteCount);
	return ByteCount;
}

HORDE_API void WriteUnsignedVarInt(FBlobWriter& Writer, size_t Value)
{
	size_t ByteCount = MeasureUnsignedVarInt(Value);
	WriteUnsignedVarIntWithKnownLength(Writer.GetOutputBuffer(ByteCount), Value, ByteCount);
	Writer.Advance(ByteCount);
}

HORDE_API void WriteUnsignedVarIntWithKnownLength(void* Buffer, size_t Value, size_t NumBytes)
{
	uint8* ByteBuffer = (uint8*)Buffer;
	for (size_t Idx = 1; Idx < NumBytes; Idx++)
	{
		ByteBuffer[NumBytes - Idx] = (unsigned)Value;
		Value >>= 8;
	}
	ByteBuffer[0] = (unsigned char)((0xff << (9 - (int)NumBytes)) | (unsigned char)Value);
}

HORDE_API size_t MeasureString(const char* Text)
{
	return MeasureString(FUtf8StringView((const UTF8CHAR*)Text));
}

HORDE_API size_t MeasureString(const FUtf8StringView& Text)
{
	return MeasureUnsignedVarInt(Text.Len()) + Text.Len();
}

HORDE_API void WriteString(FBlobWriter& Writer, const char* Text)
{
	WriteString(Writer, FUtf8StringView(Text));
}

HORDE_API void WriteString(FBlobWriter& Writer, const FUtf8StringView& Text)
{
	WriteUnsignedVarInt(Writer, Text.Len());
	WriteFixedLengthBytes(Writer, Text.GetData(), Text.Len());
}

HORDE_API void WriteString(FBlobWriter& Writer, const FUtf8String& Text)
{
	WriteString(Writer, FUtf8StringView(Text));
}
