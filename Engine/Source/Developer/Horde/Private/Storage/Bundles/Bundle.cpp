// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/Bundles/Bundle.h"

const FBlobType FBundle::BlobType(FGuid(0x7C5BA294, 0x4F922D21, 0x2F85BE85, 0x1E4CCC48), 1);

FBundleSignature::FBundleSignature(EBundleVersion InVersion, size_t InLength)
	: Version(InVersion)
	, Length(InLength)
{
}

FBundleSignature FBundleSignature::Read(const void* InData)
{
	const unsigned char* Data = (const unsigned char*)InData;
	if (Data[0] == 'U' && Data[1] == 'E' && Data[2] == 'B' && Data[3] == 'N')
	{
		EBundleVersion Version = EBundleVersion::Initial;
		size_t Length = (Data[4] << 24) | (Data[5] << 16) | (Data[6] << 8) | Data[7];
		return FBundleSignature(Version, Length);
	}
	else if (Data[0] == 'U' && Data[1] == 'B' && Data[2] == 'N')
	{
		EBundleVersion Version = (EBundleVersion)Data[3];
		size_t Length = *(const int*)(Data + 4);
		return FBundleSignature(Version, Length);
	}
	else
	{
		check(false);
		return FBundleSignature((EBundleVersion)0, 0);
	}
}

void FBundleSignature::Write(void* InData)
{
	unsigned char* Data = (unsigned char*)InData;
	if (Version == EBundleVersion::Initial)
	{
		Data[0] = 'U';
		Data[1] = 'E';
		Data[2] = 'B';
		Data[3] = 'N';

		Data[4] = (unsigned char)(Length >> 24);
		Data[5] = (unsigned char)(Length >> 16);
		Data[6] = (unsigned char)(Length >> 8);
		Data[7] = (unsigned char)(Length);
	}
	else
	{
		Data[0] = 'U';
		Data[1] = 'B';
		Data[2] = 'N';
		Data[3] = (unsigned char)Version;

		*(int*)(Data + 4) = (int)Length;
	}
}
