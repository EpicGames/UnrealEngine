// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/Bundles/BundleCompression.h"
#include "HordePlatform.h"
#include "Misc/Compression.h"

size_t FBundleCompression::GetMaxSize(EBundleCompressionFormat Format, const FMemoryView& Input)
{
	switch (Format)
	{
	case EBundleCompressionFormat::None:
		return Input.GetSize();
	case EBundleCompressionFormat::LZ4:
		return FCompression::CompressMemoryBound(NAME_LZ4, Input.GetSize());
	case EBundleCompressionFormat::Gzip:
		return FCompression::CompressMemoryBound(NAME_Gzip, Input.GetSize());
	case EBundleCompressionFormat::Oodle:
		return FCompression::CompressMemoryBound(NAME_Oodle, Input.GetSize());
	default:
		FHordePlatform::NotSupported("The specified compression format is not currently supported");
	}
}

size_t FBundleCompression::Compress(EBundleCompressionFormat Format, const FMemoryView& Input, const FMutableMemoryView& Output)
{
	int32 CompressedSize = Output.GetSize();
	switch (Format)
	{
	case EBundleCompressionFormat::None:
		check(Output.GetSize() == Input.GetSize());
		memcpy(Output.GetData(), Input.GetData(), Input.GetSize());
		return Input.GetSize();
	case EBundleCompressionFormat::LZ4:
		verify(FCompression::CompressMemory(NAME_LZ4, Output.GetData(), CompressedSize, Input.GetData(), Input.GetSize()));
		return CompressedSize;
	case EBundleCompressionFormat::Gzip:
		verify(FCompression::CompressMemory(NAME_Gzip, Output.GetData(), CompressedSize, Input.GetData(), Input.GetSize()));
		return CompressedSize;
	case EBundleCompressionFormat::Oodle:
		verify(FCompression::CompressMemory(NAME_Oodle, Output.GetData(), CompressedSize, Input.GetData(), Input.GetSize()));
		return CompressedSize;
	default:
		FHordePlatform::NotSupported("The specified compression format is not currently supported");
	}
}

void FBundleCompression::Decompress(EBundleCompressionFormat Format, const FMemoryView& Input, const FMutableMemoryView& Output)
{
	switch (Format)
	{
	case EBundleCompressionFormat::None:
		memcpy(Output.GetData(), Input.GetData(), Input.GetSize());
		break;
	case EBundleCompressionFormat::LZ4:
		verify(FCompression::UncompressMemory(NAME_LZ4, Output.GetData(), Output.GetSize(), Input.GetData(), Input.GetSize()));
		break;
	case EBundleCompressionFormat::Gzip:
		verify(FCompression::UncompressMemory(NAME_Gzip, Output.GetData(), Output.GetSize(), Input.GetData(), Input.GetSize()));
		break;
	case EBundleCompressionFormat::Oodle:
		verify(FCompression::UncompressMemory(NAME_Oodle, Output.GetData(), Output.GetSize(), Input.GetData(), Input.GetSize()));
		break;
	default:
		FHordePlatform::NotSupported("The specified compression format is not currently supported");
	}
}
