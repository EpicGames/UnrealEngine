// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/Bundles/V2/BundleWriter.h"
#include "HordePlatform.h"
#include "Storage/Bundles/V2/PacketWriter.h"
#include "Storage/Blob.h"
#include "Storage/StorageClient.h"
#include "Storage/Bundles/V2/ExportHandle.h"
#include "Storage/ChunkedBufferWriter.h"

//
// Data for an export that has been written to storage, but may not have been flushed yet
//
class FBundleWriter::FPendingExportHandleData : public FBlobHandleData
{
public:
	static const char HandleType[];

	FPendingExportHandleData(TBlobHandle<FPendingPacketHandleData> InPacket, int32 InExportIdx);

	// Overridden from FBlobHandleData
	bool Equals(const FBlobHandleData& Other) const override;
	uint32 GetHashCode() const override;
	FBlobHandle GetOuter() const override;
	const char* GetType() const override;
	FBlob Read() const override;
	bool TryAppendIdentifier(FUtf8String& OutIdentifier) const override;

private:
	TBlobHandle<FPendingPacketHandleData> Packet;
	const int32 ExportIdx;
};

//
// Packet which may not have been flushed yet
//
class FBundleWriter::FPendingPacketHandleData : public FBlobHandleData
{
public:
	static const char HandleType[];

	FPendingPacketHandleData(FBundleWriter& InBundleWriter, TBlobHandle<FPendingBundleHandleData> InBundle);

	// Reads an export from the packet. The packet may or may not be flushed.
	FBlob ReadExport(int ExportIdx) const;

	// Marks the current packet as complete, and returns the packet writer for reuse on the next packet
	void MarkComplete(TBlobHandle<FPacketHandleData> InFlushedHandle);

	// Overridden from FBlobHandleData
	bool Equals(const FBlobHandleData& Other) const override;
	uint32 GetHashCode() const override;
	FBlobHandle GetOuter() const override;
	const char* GetType() const override;
	FBlob Read() const override;
	bool TryAppendIdentifier(FUtf8String& OutIdentifier) const override;

private:
	FBundleWriter* BundleWriter;
	TBlobHandle<FPendingBundleHandleData> BundleHandle;
	TBlobHandle<FPacketHandleData> FlushedHandle;
};

//
// Bundle which is still being written to
//
class FBundleWriter::FPendingBundleHandleData : public FBlobHandleData, public TSharedFromThis<FBundleWriter::FPendingBundleHandleData>
{
public:
	static const char HandleType[];

	mutable FCriticalSection CriticalSection;

	FPendingBundleHandleData(FBundleWriter& InBundleWriter);
	void MarkComplete(FBlobHandle InFlushedHandle);

	// Overridden from FBlobHandleData
	bool Equals(const FBlobHandleData& Other) const override;
	uint32 GetHashCode() const override;
	FBlobHandle GetOuter() const override;
	const char* GetType() const override;
	FBlob Read() const override;
	FSharedBufferView ReadBody(size_t Offset, TOptional<size_t> Length = TOptional<size_t>()) const override;
	bool TryAppendIdentifier(FUtf8String& OutIdentifier) const override;

private:
	FBundleWriter* BundleWriter;
	FBlobHandle FlushedHandle;
};





// --------------------------------------------------------------------------------------------------------

const char FBundleWriter::FPendingExportHandleData::HandleType[] = "FBundleWriter::FPendingExportHandleData";

FBundleWriter::FPendingExportHandleData::FPendingExportHandleData(TBlobHandle<FPendingPacketHandleData> InPacket, int32 InExportIdx)
	: Packet(MoveTemp(InPacket))
	, ExportIdx(InExportIdx)
{
}

bool FBundleWriter::FPendingExportHandleData::Equals(const FBlobHandleData& Other) const
{
	return this == &Other;
}

uint32 FBundleWriter::FPendingExportHandleData::GetHashCode() const
{
	return (uint32)(size_t)this;
}

FBlobHandle FBundleWriter::FPendingExportHandleData::GetOuter() const
{
	return Packet;
}

const char* FBundleWriter::FPendingExportHandleData::GetType() const
{
	return HandleType;
}

FBlob FBundleWriter::FPendingExportHandleData::Read() const
{
	return Packet->ReadExport(ExportIdx);
}

bool FBundleWriter::FPendingExportHandleData::TryAppendIdentifier(FUtf8String& OutIdentifier) const
{
	FExportHandleData::AppendIdentifier(OutIdentifier, ExportIdx);
	return true;
}






// --------------------------------------------------------------------------------------------------------

const char FBundleWriter::FPendingPacketHandleData::HandleType[] = "FBundleWriter::FPendingPacketHandleData";

FBundleWriter::FPendingPacketHandleData::FPendingPacketHandleData(FBundleWriter& InBundleWriter, TBlobHandle<FPendingBundleHandleData> InBundleHandle)
	: BundleWriter(&InBundleWriter)
	, BundleHandle(MoveTemp(InBundleHandle))
{
}

void FBundleWriter::FPendingPacketHandleData::MarkComplete(TBlobHandle<FPacketHandleData> InFlushedHandle)
{
	check(!FlushedHandle);
	BundleWriter = nullptr;
	FlushedHandle = MoveTemp(InFlushedHandle);
}

bool FBundleWriter::FPendingPacketHandleData::Equals(const FBlobHandleData& Other) const
{
	return this == &Other;
}

uint32 FBundleWriter::FPendingPacketHandleData::GetHashCode() const
{
	return (uint32)(size_t)this;
}

FBlobHandle FBundleWriter::FPendingPacketHandleData::GetOuter() const
{
	return FlushedHandle.IsValid() ? (FBlobHandle)FlushedHandle->GetOuter() : (FBlobHandle)BundleHandle;
}

const char* FBundleWriter::FPendingPacketHandleData::GetType() const
{
	return HandleType;
}

FBlob FBundleWriter::FPendingPacketHandleData::Read() const
{
	{
		FScopeLock Lock(&BundleHandle->CriticalSection);
		check(FlushedHandle);
	}
	return FlushedHandle->Read();
}

FBlob FBundleWriter::FPendingPacketHandleData::ReadExport(int ExportIdx) const
{
	{
		FScopeLock Lock(&BundleHandle->CriticalSection);
		if (!FlushedHandle)
		{
			// If not flushed yet, the packet writer must be this packet
			check(BundleWriter->CurrentPacketHandle.Get() == this);
			return BundleWriter->PacketWriter->GetExport(ExportIdx);
		}
	}
	return FlushedHandle->ReadExport(ExportIdx);
}

bool FBundleWriter::FPendingPacketHandleData::TryAppendIdentifier(FUtf8String& OutIdentifier) const
{
	if (FlushedHandle)
	{
		return FlushedHandle->TryAppendIdentifier(OutIdentifier);
	}
	else
	{
		return false;
	}
}





// --------------------------------------------------------------------------------------------------------

const char FBundleWriter::FPendingBundleHandleData::HandleType[] = "FBundleWriter::FPendingBundleHandleData";

FBundleWriter::FPendingBundleHandleData::FPendingBundleHandleData(FBundleWriter& InBundleWriter)
	: BundleWriter(&InBundleWriter)
{
}

void FBundleWriter::FPendingBundleHandleData::MarkComplete(FBlobHandle InFlushedHandle)
{
	FScopeLock Lock(&CriticalSection);
	check(!FlushedHandle);
	FlushedHandle = InFlushedHandle;
	BundleWriter = nullptr;
}

bool FBundleWriter::FPendingBundleHandleData::Equals(const FBlobHandleData& Other) const
{
	return this == &Other;
}

uint32 FBundleWriter::FPendingBundleHandleData::GetHashCode() const
{
	return (uint32)(size_t)this;
}

FBlobHandle FBundleWriter::FPendingBundleHandleData::GetOuter() const
{
	return FBlobHandle();
}

const char* FBundleWriter::FPendingBundleHandleData::GetType() const
{
	return HandleType;
}

FBlob FBundleWriter::FPendingBundleHandleData::Read() const
{
	return FlushedHandle->Read();
}

FSharedBufferView FBundleWriter::FPendingBundleHandleData::ReadBody(size_t Offset, TOptional<size_t> Length) const
{
	if (!FlushedHandle)
	{
		FScopeLock Lock(&CriticalSection);
		if (!FlushedHandle)
		{
			size_t LengthValue = Length.IsSet() ? Length.GetValue() : BundleWriter->CompressedPacketWriter.GetLength() - Offset;
			return BundleWriter->CompressedPacketWriter.Slice(Offset, LengthValue);
		}
	}
	return FlushedHandle->ReadBody(Offset, Length);
}

bool FBundleWriter::FPendingBundleHandleData::TryAppendIdentifier(FUtf8String& OutIdentifier) const
{
	if (FlushedHandle)
	{
		FlushedHandle->TryAppendIdentifier(OutIdentifier);
		return true;
	}
	return false;
}








// --------------------------------------------------------------------------------------------------------

FBundleWriter::FBundleWriter(TSharedRef<FKeyValueStorageClient> InStorageClient, FUtf8String InBasePath, const FBundleOptions& InOptions)
	: StorageClient(MoveTemp(InStorageClient))
	, BasePath(MoveTemp(InBasePath))
	, Options(InOptions)
	, CurrentBundleHandle(MakeShared<FPendingBundleHandleData>(*this))
{
	StartPacket();
}

FBundleWriter::~FBundleWriter()
{
}

void FBundleWriter::AddAlias(const FAliasInfo& Alias)
{
	FHordePlatform::NotImplemented();
}

void FBundleWriter::AddImport(FBlobHandle Handle)
{
	PacketWriter->AddImport(MoveTemp(Handle));
}

void FBundleWriter::AddRef(const FRefName& RefName, const FRefOptions& RefOptions)
{
	FHordePlatform::NotImplemented();
}

FBlobHandle FBundleWriter::CompleteBlob(const FBlobType& Type)
{
	check(PacketWriter);
	check(CurrentPacketHandle);
	check(CurrentBundleHandle);

	int ExportIdx = PacketWriter->CompleteBlob(Type);
	FBlobHandle ExportHandle(MakeShared<FPendingExportHandleData>(CurrentPacketHandle, ExportIdx));

//	for (FAliasInfo& Alias : Aliases)
//	{
//		PendingExportAliases.push_back(std::make_pair(ExportHandle, std::move(Alias)));
//	}

	if (PacketWriter->GetLength() > FMath::Min(Options.MinCompressionPacketSize, Options.MaxBlobSize))
	{
		FinishPacket();
		StartPacket();
	}

	return ExportHandle;
}

FMutableMemoryView FBundleWriter::GetOutputBufferAsSpan(size_t UsedSize, size_t DesiredSize)
{
	return PacketWriter->GetOutputBuffer(UsedSize, DesiredSize);
}

void FBundleWriter::Advance(size_t Size)
{
	PacketWriter->Advance(Size);
}

void FBundleWriter::Flush()
{
	// Check we haven't already flushed this bundle
	if (CurrentPacketHandle)
	{
		FinishPacket();
	}
	if (CompressedPacketWriter.GetLength() == 0)
	{
		return;
	}

	// Write the data to the underlying store
	TArray<FMemoryView> View = CompressedPacketWriter.GetView();
	FBlobHandle FlushedHandle = StorageClient->WriteBlob(BasePath, FBundle::BlobType, View, CurrentBundleImports);

	// Update the bundle and packet handles
	{
		FScopeLock Lock(&CurrentBundleHandle->CriticalSection);
		CurrentBundleHandle->MarkComplete(FlushedHandle);
		CurrentBundleHandle = MakeShared<FPendingBundleHandleData>(*this);
	}

	// Reset the writer
	CompressedPacketWriter.Reset();
	CurrentBundleImports.Empty();

	// Create a new bundle

	// TODO: put all the encoded packets into the cache using the final handles

//	// Add all the aliases
//	for(std::pair<FBlobHandle, FAliasInfo>& PendingExportAlias : PendingExportAliases)
//	{
//		FBlobAlias Alias(PendingExportAlias.first, PendingExportAlias.second.Rank, PendingExportAlias.second.Data);
//		StorageClient->AddAlias(PendingExportAlias.second.Name.c_str(), MoveTemp(Alias));
//	}
}

TUniquePtr<FBlobWriter> FBundleWriter::Fork()
{
	return TUniquePtr<FBlobWriter>(new FBundleWriter(StorageClient, BasePath, Options));
}

void FBundleWriter::StartPacket()
{
	check(CurrentBundleHandle);
	check(!CurrentPacketHandle);
	check(!PacketWriter);

	CurrentPacketHandle = MakeShared<FPendingPacketHandleData>(*this, CurrentBundleHandle);
	PacketWriter = MakeUnique<FPacketWriter>(CurrentBundleHandle, CurrentPacketHandle);
}

void FBundleWriter::FinishPacket()
{
	check(CurrentBundleHandle);
	check(CurrentPacketHandle);
	check(PacketWriter);

	if (PacketWriter->GetExportCount() > 0)
	{
		size_t PacketOffset = CompressedPacketWriter.GetLength();
		FPacket Packet = PacketWriter->CompletePacket();
		Packet.Encode(Options.CompressionFormat, CompressedPacketWriter);
		size_t PacketLength = CompressedPacketWriter.GetLength() - PacketOffset;

		// Point the packet handle to the encoded data
		{
			FScopeLock Lock(&CurrentBundleHandle->CriticalSection);
			TBlobHandle<FPacketHandleData> FlushedPacketHandle = MakeShared<FPacketHandleData>(StorageClient, CurrentBundleHandle, PacketOffset, PacketLength);
			CurrentPacketHandle->MarkComplete(MoveTemp(FlushedPacketHandle));
		}

		// Find all the other bundles that are referenced
		for (int ImportIdx = 0; ImportIdx < Packet.GetImportCount(); ImportIdx++)
		{
			FPacketImport Import = Packet.GetImport(ImportIdx);
			if (Import.GetBaseIdx() == FPacketImport::InvalidBaseIdx)
			{
				CurrentBundleImports.Add(PacketWriter->GetImport(ImportIdx));
			}
		}
	}

	PacketWriter.Reset();
	CurrentPacketHandle = TBlobHandle<FPendingPacketHandleData>();
}
