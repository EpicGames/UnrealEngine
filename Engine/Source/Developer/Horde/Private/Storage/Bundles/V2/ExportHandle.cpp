// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/Bundles/V2/ExportHandle.h"
#include "HordePlatform.h"

const char FExportHandleData::Type[] = "FExportHandleData";
const FUtf8StringView FExportHandleData::FragmentPrefix("exp=");

FExportHandleData::FExportHandleData(FPacketHandle InPacket, int32 InExportIdx)
	: Packet(MoveTemp(InPacket))
	, ExportIdx(InExportIdx)
{
}

FExportHandleData::FExportHandleData(FPacketHandle InPacket, const FUtf8StringView& InFragment)
	: Packet(MoveTemp(InPacket))
{
	verify(TryParse(InFragment, ExportIdx));
}

FExportHandleData::~FExportHandleData()
{
}

void FExportHandleData::AppendIdentifier(FUtf8String& OutBuffer, int32 ExportIdx)
{
	OutBuffer.Appendf("exp=%d", ExportIdx);
}

bool FExportHandleData::Equals(const FBlobHandleData& Other) const
{
	if (GetType() != Other.GetType())
	{
		return false;
	}

	const FExportHandleData& OtherExport = (const FExportHandleData&)Other;
	return Packet->Equals(*OtherExport.Packet) && ExportIdx == OtherExport.ExportIdx;
}

uint32 FExportHandleData::GetHashCode() const
{
	return (Packet->GetHashCode() * 257) + ExportIdx;
}

FBlobHandle FExportHandleData::GetOuter() const
{
	return Packet;
}

const char* FExportHandleData::GetType() const
{
	return Type;
}

FBlob FExportHandleData::Read() const
{
	return Packet->ReadExport(ExportIdx);
}

bool FExportHandleData::TryAppendIdentifier(FUtf8String& OutBuffer) const
{
	AppendIdentifier(OutBuffer, ExportIdx);
	return true;
}

bool FExportHandleData::TryParse(const FUtf8StringView& Fragment, int32& OutExportIdx)
{
	OutExportIdx = 0;

	if (!Fragment.StartsWith(FragmentPrefix, ESearchCase::CaseSensitive))
	{
		return false;
	}

	const UTF8CHAR* Data = Fragment.GetData();
	for (size_t Idx = FragmentPrefix.Len(); Idx < Fragment.Len(); Idx++)
	{
		UTF8CHAR Char = Data[Idx];
		if (Char >= '0' && Char <= '9')
		{
			OutExportIdx = (OutExportIdx * 10) + (Char - '0');
		}
		else
		{
			return false;
		}
	}

	return true;
}
