// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/Bundles/V2/Packet.h"
#include "Storage/Bundles/Bundle.h"
#include "Storage/BlobWriter.h"
#include "Storage/ChunkedBufferWriter.h"
#include "Serialization/VarInt.h"
#include "HordePlatform.h"

const FBlobType FPacket::BlobType(FGuid(0xCD9A04EF, 0x47D3CAC1, 0x492A05A6, 0x51E63081), 1);

FPacket::FPacket(FSharedBufferView InBuffer)
	: Buffer(MoveTemp(InBuffer))
{
}

FPacket::~FPacket()
{
}

FSharedBufferView FPacket::GetBuffer() const
{
	return Buffer;
}

size_t FPacket::GetLength() const
{
	return Buffer.GetLength();
}

size_t FPacket::GetTypeCount() const
{
	const unsigned char* Data = Buffer.GetPointer();
	int TypeTableOffset = *(const int*)(Data + 8);
	return *(int*)(Data + TypeTableOffset);
}

const FBlobType& FPacket::GetType(size_t TypeIdx) const
{
	const unsigned char* Data = Buffer.GetPointer();
	int TypeTableOffset = *(const int*)(Data + 8);

	int TypeCount = *(Data + TypeTableOffset);
	check(TypeIdx >= 0 && TypeIdx < TypeCount);

	const FBlobType* Types = (const FBlobType*)(Data + TypeTableOffset + sizeof(int));
	return Types[TypeIdx];
}

size_t FPacket::GetImportCount() const
{
	const unsigned char* Data = Buffer.GetPointer();
	int ImportTableOffset = *(const int*)(Data + 12);
	return *(const int*)(Data + ImportTableOffset);
}

FPacketImport FPacket::GetImport(size_t ImportIdx) const
{
	const unsigned char* Data = Buffer.GetPointer();
	size_t ImportTableOffset = *(const int*)(Data + 12);

	size_t ImportCount = *(const int*)(Data + ImportTableOffset);
	check(ImportIdx >= 0 && ImportIdx < ImportCount);

	size_t ImportEntryOffset = ImportTableOffset + sizeof(int) + ImportIdx * sizeof(int);

	size_t ImportOffset = *(const int*)(Data + ImportEntryOffset);
	size_t ImportLength = *(const int*)(Data + ImportEntryOffset + 4) - ImportOffset;

	return FPacketImport(Buffer.GetView().Mid(ImportOffset, ImportLength));
}

size_t FPacket::GetExportCount() const
{
	const unsigned char* Data = Buffer.GetPointer();
	size_t ExportTableOffset = *(const int*)(Data + 16);
	return *(const int*)(Data + ExportTableOffset);
}

FPacketExport FPacket::GetExport(size_t ExportIdx) const
{
	const unsigned char* Data = Buffer.GetPointer();
	size_t ExportTableOffset = *(const int*)(Data + 16);

	size_t ExportCount = *(const int*)(Data + ExportTableOffset);
	check(ExportIdx >= 0 && ExportIdx < ExportCount);

	size_t ExportEntryOffset = ExportTableOffset + sizeof(int) + ExportIdx * sizeof(int);

	size_t ExportOffset = *(const int*)(Data + ExportEntryOffset);
	size_t ExportLength = *(const int*)(Data + ExportEntryOffset + 4) - ExportOffset;

	return FPacketExport(Buffer.Slice(ExportOffset, ExportLength));
}

void FPacket::Encode(EBundleCompressionFormat Format, FChunkedBufferWriter& Writer)
{
	FMemoryView View = Buffer.GetView();

	size_t HeaderLength = FBundleSignature::NumBytes + 5;
	FMutableMemoryView Output = Writer.GetOutputBuffer(0, HeaderLength + FBundleCompression::GetMaxSize(Format, View));

	unsigned char* OutputData = (unsigned char*)Output.GetData();
	size_t EncodedLength = FBundleCompression::Compress(Format, View, Output.Mid(HeaderLength));

	FBundleSignature(EBundleVersion::LatestV2, HeaderLength + EncodedLength).Write(OutputData);
	OutputData += FBundleSignature::NumBytes;

	*(int*)OutputData = (int)GetLength();
	OutputData += sizeof(int);

	*OutputData = (unsigned char)Format;

	Writer.Advance(HeaderLength + EncodedLength);
}

FPacket FPacket::Decode(const FMemoryView& View)
{
	const unsigned char* Data = (const unsigned char*)View.GetData();

	FBundleSignature Signature = FBundleSignature::Read(Data);
	Data += FBundleSignature::NumBytes;

	if (Signature.Version <= EBundleVersion::LatestV1 || Signature.Version > EBundleVersion::LatestV2)
	{
		FHordePlatform::NotSupported("Unsupported bundle version");
	}

	int DecodedLength = *(int*)Data;
	Data += 4;

	EBundleCompressionFormat Format = (EBundleCompressionFormat)*Data;
	Data++;

	FUniqueBuffer Buffer = FUniqueBuffer::Alloc(DecodedLength);
	FBundleCompression::Decompress(Format, View.Left(Signature.Length).Mid(FBundleSignature::NumBytes + 5), Buffer.GetView());

	return FPacket(Buffer.MoveToShared());
}

// --------------------------------------------------------------------------------------------------------

FPacketImport::FPacketImport(const FMemoryView& InView)
{
	uint32 ByteCount;
	BaseIdx = (int64)ReadVarUInt(InView.GetData(), ByteCount) - Bias;

	FMemoryView FragmentData = InView.Mid(ByteCount);
	Fragment = FUtf8StringView((const UTF8CHAR*)FragmentData.GetData(), FragmentData.GetSize());
}

FPacketImport::FPacketImport(int32 InBaseIdx, const FUtf8StringView& InFragment)
	: BaseIdx(InBaseIdx)
	, Fragment(InFragment)
{
}

int FPacketImport::GetBaseIdx() const
{
	return BaseIdx;
}

FUtf8StringView FPacketImport::GetFragment() const
{
	return Fragment;
}

// --------------------------------------------------------------------------------------------------------

FPacketExport::FPacketExport(FSharedBufferView InBuffer)
	: Buffer(MoveTemp(InBuffer))
{
	const char* Data = (const char*)Buffer.GetPointer();

	uint32 Offset = sizeof(uint32) + *(const uint32*)Data;
	uint32 ByteCount;

	TypeIdx = (int32)ReadVarUInt(Data + Offset, ByteCount);
	Offset += ByteCount;

	Imports = InBuffer.GetView().Mid(Offset);
}

FPacketExport::~FPacketExport()
{
}

size_t FPacketExport::GetTypeIndex() const
{
	return TypeIdx;
}

void FPacketExport::GetImportIndices(TArray<size_t>& OutIndices) const
{
	uint32 ByteCount;
	const char* Data = (const char*)Imports.GetData();

	size_t NumImports = ReadVarUInt(Data, ByteCount);
	Data += ByteCount;
	OutIndices.Reserve(OutIndices.Num() + NumImports);

	for (size_t Idx = 0; Idx < NumImports; Idx++)
	{
		size_t Index = ReadVarUInt(Data, ByteCount);
		Data += ByteCount;
		OutIndices.Add(Index);
	}
}

FSharedBufferView FPacketExport::GetPayload() const
{
	int PayloadLength = *(int*)Buffer.GetPointer();
	return Buffer.Slice(sizeof(int), PayloadLength);
}
