// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/Bundles/V2/PacketHandle.h"
#include "Storage/Bundles/V2/PacketReader.h"
#include "Storage/Bundles/V2/ExportHandle.h"
#include "HordePlatform.h"

// --------------------------------------------------------------------------------------------------------

const char FPacketHandleData::Type[] = "FPacketHandleDetail";
const char FPacketHandleData::FragmentPrefix[] = "pkt=";
const size_t FPacketHandleData::FragmentPrefixLength = (sizeof(FragmentPrefix) / sizeof(FragmentPrefix[0])) - 1;

FPacketHandleData::FPacketHandleData(TSharedRef<FStorageClient> InStorageClient, FBlobHandle InBundle, size_t InPacketOffset, size_t InPacketLength)
	: StorageClient(MoveTemp(InStorageClient))
	, Bundle(MoveTemp(InBundle))
	, PacketOffset(InPacketOffset)
	, PacketLength(InPacketLength)
{
}

FPacketHandleData::FPacketHandleData(TSharedRef<FStorageClient> InStorageClient, FBlobHandle InBundle, const FUtf8StringView& InFragment)
	: StorageClient(MoveTemp(InStorageClient))
	, Bundle(MoveTemp(InBundle))
{
	verify(TryParse(InFragment, PacketOffset, PacketLength));
}

FPacketHandleData::~FPacketHandleData()
{
}

FBlob FPacketHandleData::ReadExport(size_t ExportIdx) const
{
	return GetPacketReader().ReadExport(ExportIdx);
}

FSharedBufferView FPacketHandleData::ReadExportBody(size_t ExportIdx) const
{
	return GetPacketReader().ReadExportBody(ExportIdx);
}

bool FPacketHandleData::Equals(const FBlobHandleData& Other) const
{
	if (GetType() != Other.GetType())
	{
		return false;
	}

	const FPacketHandleData& OtherPacket = (const FPacketHandleData&)Other;
	return Bundle->Equals(*OtherPacket.Bundle) && PacketOffset == OtherPacket.PacketOffset;
}

uint32 FPacketHandleData::GetHashCode() const
{
	return (Bundle->GetHashCode() * 257) + PacketOffset;
}

FBlobHandle FPacketHandleData::GetOuter() const
{
	return Bundle;
}

const char* FPacketHandleData::GetType() const
{
	return Type;
}

FBlob FPacketHandleData::Read() const
{
	const FPacketReader& PacketReaderInst = GetPacketReader();
	return PacketReaderInst.ReadPacket();
}

bool FPacketHandleData::TryAppendIdentifier(FUtf8String& OutBuffer) const
{
	OutBuffer.Appendf("pkt=%zu,%zu", PacketOffset, PacketLength);
	return true;
}

FBlobHandle FPacketHandleData::GetFragmentHandle(const FUtf8StringView& Fragment) const
{
	TBlobHandle<FPacketHandleData> PacketHandle(const_cast<FPacketHandleData*>(this)->AsShared());
	return MakeShared<FExportHandleData>(PacketHandle, Fragment);
}

const FPacketReader& FPacketHandleData::GetPacketReader() const
{
	if (!PacketReader)
	{
		FSharedBufferView Buffer = Bundle->ReadBody(PacketOffset, PacketLength);
		FPacket Packet = FPacket::Decode(Buffer.GetView());
		PacketReader = MakeShared<FPacketReader>(const_cast<FPacketHandleData*>(this)->StorageClient, Bundle, const_cast<FPacketHandleData*>(this)->AsShared(), MoveTemp(Packet));
	}
	return *PacketReader.Get();
}

bool FPacketHandleData::TryParse(const FUtf8StringView& Fragment, size_t& OutPacketOffset, size_t& OutPacketLength)
{
	size_t NumBytesRead;
	if (!Fragment.StartsWith(FragmentPrefix))
	{
		OutPacketOffset = OutPacketLength = 0;
		return false;
	}

	FUtf8StringView Remaining = Fragment.Mid(FragmentPrefixLength);
	if (!FHordePlatform::TryParseSizeT((const char*)Remaining.GetData(), Remaining.Len(), OutPacketOffset, NumBytesRead))
	{
		OutPacketOffset = OutPacketLength = 0;
		return false;
	}

	Remaining = Remaining.Mid(NumBytesRead);
	if (Remaining.Len() == 0 || Remaining[0] != ',')
	{
		OutPacketOffset = OutPacketLength = 0;
		return false;
	}

	Remaining = Remaining.Mid(1);
	if (!FHordePlatform::TryParseSizeT((const char*)Remaining.GetData(), Remaining.Len(), OutPacketLength, NumBytesRead) || NumBytesRead != Remaining.Len())
	{
		OutPacketOffset = OutPacketLength = 0;
		return false;
	}

	return true;
}
