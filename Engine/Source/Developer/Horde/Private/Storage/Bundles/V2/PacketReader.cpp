// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/Bundles/V2/PacketReader.h"
#include "Storage/Bundles/V2/ExportHandle.h"
#include "Storage/Bundles/V2/PacketHandle.h"
#include "Storage/Blob.h"
#include "HordePlatform.h"

FPacketReader::FPacketReader(TSharedRef<FStorageClient> InStorageClient, FBlobHandle InBundleHandle, FPacketHandle InPacketHandle, FPacket InPacket)
	: StorageClient(MoveTemp(InStorageClient))
	, BundleHandle(MoveTemp(InBundleHandle))
	, PacketHandle(MoveTemp(InPacketHandle))
	, Packet(MoveTemp(InPacket))
{
	CachedImportHandles.SetNum(Packet.GetImportCount());
}

FPacketReader::~FPacketReader()
{
}

FBlob FPacketReader::ReadPacket() const
{
	for (int Idx = 0; Idx < CachedImportHandles.Num(); Idx++)
	{
		GetImportHandle(Idx);
	}
	return FBlob(FPacket::BlobType, Packet.GetBuffer(), CachedImportHandles);
}

FBlob FPacketReader::ReadExport(size_t ExportIdx) const
{
	FPacketExport Export = Packet.GetExport(ExportIdx);

	FBlobType Type = Packet.GetType(Export.GetTypeIndex());
	FSharedBufferView Data = Export.GetPayload();

	TArray<size_t> ImportIndices;
	Export.GetImportIndices(ImportIndices);

	TArray<FBlobHandle> ImportHandles;
	ImportHandles.Reserve(ImportIndices.Num());

	for (size_t ImportIndex : ImportIndices)
	{
		ImportHandles.Add(GetImportHandle(ImportIndex));
	}

	return FBlob(Type, Export.GetPayload(), MoveTemp(ImportHandles));
}

FSharedBufferView FPacketReader::ReadExportBody(size_t ExportIdx) const
{
	FPacketExport Export = Packet.GetExport(ExportIdx);
	return Export.GetPayload();
}

FBlobHandle FPacketReader::GetImportHandle(size_t Index) const
{
	FBlobHandle ImportHandle = CachedImportHandles[Index];
	if (!ImportHandle)
	{
		FPacketImport Import = Packet.GetImport(Index);

		size_t BaseIdx = Import.GetBaseIdx();
		switch (BaseIdx)
		{
		case FPacketImport::InvalidBaseIdx:
			ImportHandle = StorageClient->CreateHandle(FBlobLocator(FUtf8String(Import.GetFragment())));
			break;
		case FPacketImport::CurrentBundleBaseIdx:
			ImportHandle = MakeShared<FPacketHandleData>(StorageClient, BundleHandle, Import.GetFragment());
			break;
		case FPacketImport::CurrentPacketBaseIdx:
			ImportHandle = MakeShared<FExportHandleData>(PacketHandle, Import.GetFragment());
			break;
		default:
			FBlobHandle ParentHandle = GetImportHandle(BaseIdx);
			ImportHandle = ParentHandle->GetFragmentHandle(Import.GetFragment());
			break;
		}

		CachedImportHandles[Index] = ImportHandle;
	}
	return ImportHandle;
}
