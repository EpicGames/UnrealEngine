// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/Bundles/V2/PacketWriter.h"
#include "Storage/Bundles/Bundle.h"
#include "Storage/BlobWriter.h"
#include "Memory/SharedBuffer.h"
#include "HordePlatform.h"

FPacketWriter::FPacketWriter(FBlobHandle InBundleHandle, FBlobHandle InPacketHandle)
	: BundleHandle(MoveTemp(InBundleHandle))
	, PacketHandle(MoveTemp(InPacketHandle))
	, Buffer(FUniqueBuffer::Alloc(1024).MoveToShared())
{
	ImportHandles.SetNum(FPacketImport::Bias);
	ImportHandles[FPacketImport::CurrentBundleBaseIdx + FPacketImport::Bias] = BundleHandle;
	ImportHandles[FPacketImport::CurrentPacketBaseIdx + FPacketImport::Bias] = PacketHandle;

	ImportMap.Add(BundleHandle, FPacketImport::CurrentBundleBaseIdx);
	ImportMap.Add(PacketHandle, FPacketImport::CurrentPacketBaseIdx);

	ImportOffsets.Add(0);

	BufferBasePtr = (uint8*)Buffer.GetData();
	FBundleSignature(EBundleVersion::LatestV2, 0).Write(BufferBasePtr);

	Length = FBundleSignature::NumBytes + (sizeof(int) * 3);
	ExportOffsets.Add((uint32)Length);

	NextBlobLength = 0;// sizeof(int32);
}

FPacketWriter::~FPacketWriter()
{
}

size_t FPacketWriter::GetLength() const
{
	return Length + NextBlobLength;
}

void FPacketWriter::AddImport(FBlobHandle Import)
{
	int32 ImportIdx = FindOrAddImport(Import);
	NextBlobImports.Add(ImportIdx);
}

int FPacketWriter::GetImportCount() const
{
	return ImportHandles.Num();
}

FBlobHandle FPacketWriter::GetImport(int ImportIdx) const
{
	return ImportHandles[ImportIdx];
}

FMutableMemoryView FPacketWriter::GetOutputBuffer(size_t UsedSize, size_t DesiredSize)
{
	size_t TotalUsedSize = NextBlobLength + UsedSize;
	if (TotalUsedSize > 0)
	{
		TotalUsedSize += sizeof(int32);
	}

	FMutableMemoryView Memory = GetOutputBufferInternal(TotalUsedSize, sizeof(int32) + NextBlobLength + DesiredSize);
	return Memory.Mid(sizeof(int32) + NextBlobLength);
}

void FPacketWriter::Advance(size_t Size)
{
	NextBlobLength += Size;
}

int FPacketWriter::CompleteBlob(const FBlobType& Type)
{
	size_t TypeIdx = FindOrAddType(Type);

	// Ensure the buffer has been allocated
	GetOutputBuffer(0, 0);

	// Finalize the written export data, inserting the length at the start.
	*(int*)(BufferBasePtr + Length) = (int)NextBlobLength;
	Length += sizeof(int) + NextBlobLength;

	// Write the type and import list
	uint8* Span = (uint8*)GetOutputBufferInternal(0, (NextBlobImports.Num() + 2) * 5).GetData();
	size_t Offset = 0;

	// Write the type and import list
	Offset += WriteUnsignedVarInt(Span + Offset, TypeIdx);
	Offset += WriteUnsignedVarInt(Span + Offset, NextBlobImports.Num());

	for (int32 Import : NextBlobImports)
	{
		Offset += WriteUnsignedVarInt(Span + Offset, Import);
	}

	Length = Align(Length + Offset);

	// Reset the current export
	NextBlobLength = 0;
	NextBlobImports.Empty();

	// Write the next export offset to the buffer
	ExportOffsets.Add((uint32)Length);
	return ExportOffsets.Num() - 2;
}

int FPacketWriter::GetExportCount() const
{
	return ExportOffsets.Num() - 1;
}

FBlob FPacketWriter::GetExport(int ExportIdx) const
{
	size_t PacketOffset = ExportOffsets[ExportIdx];
	size_t PacketLength = ExportOffsets[ExportIdx + 1] - PacketOffset;

	FPacketExport Export(FSharedBufferView(Buffer, PacketOffset, PacketLength));

	FBlobType Type = Types[Export.GetTypeIndex()];

	TArray<size_t> ImportIndices;
	Export.GetImportIndices(ImportIndices);

	TArray<FBlobHandle> Imports;
	for (size_t ImportIdx : ImportIndices)
	{
		Imports.Add(ImportHandles[ImportIdx + FPacketImport::Bias]);
	}

	return FBlob(Type, Export.GetPayload(), MoveTemp(Imports));
}

FPacket FPacketWriter::CompletePacket()
{
	Length = Align(Length);

	// Write the various lookup tables
	WriteTypeTable();
	WriteImportTable();
	WriteExportTable();

	// Write the final packet length
	FBundleSignature(EBundleVersion::LatestV2, Length).Write(BufferBasePtr);
	return FPacket(FSharedBufferView(Buffer, 0, Length));
}

FMutableMemoryView FPacketWriter::GetOutputBufferInternal(size_t UsedSize, size_t DesiredSize)
{
	if (Length + DesiredSize > Buffer.GetSize())
	{
		int NewSize = (Length + DesiredSize + 4096 + 16384) & ~16384;

		FSharedBuffer NewBuffer = FUniqueBuffer::Alloc(NewSize).MoveToShared();
		uint8* NewBufferBasePtr = (uint8*)NewBuffer.GetData();
		memcpy(NewBufferBasePtr, BufferBasePtr, Length + UsedSize);

		Buffer = MoveTemp(NewBuffer);
		BufferBasePtr = NewBufferBasePtr;
	}
	return FMutableMemoryView(BufferBasePtr + Length, Buffer.GetSize() - Length);
}

unsigned char* FPacketWriter::GetOutputSpanAndAdvance(size_t Size)
{
	FMutableMemoryView Span = GetOutputBufferInternal(0, Size);
	Length += Size;
	return (uint8*)Span.GetData();
}

int32 FPacketWriter::FindOrAddType(FBlobType Type)
{
	for (int TypeIdx = 0; TypeIdx < Types.Num(); TypeIdx++)
	{
		if (Types[TypeIdx] == Type)
		{
			return TypeIdx;
		}
	}

	Types.Add(Type);
	return Types.Num() - 1;
}

int32 FPacketWriter::FindOrAddImport(const FBlobHandle& Handle)
{
	int32* ImportIdxPtr = ImportMap.Find(Handle);
	if (ImportIdxPtr == nullptr)
	{
		FUtf8String Fragment;
		verify(Handle->TryAppendIdentifier(Fragment));

		FBlobHandle Outer = Handle->GetOuter();

		int BaseIdx;
		if (Outer)
		{
			BaseIdx = FindOrAddImport(Outer);
		}
		else
		{
			BaseIdx = FPacketImport::InvalidBaseIdx;
		}

		size_t BaseIdxLen = MeasureUnsignedVarInt(BaseIdx + FPacketImport::Bias);
		size_t ImportLen = BaseIdxLen + Fragment.Len();

		FMutableMemoryView ImportData = ImportWriter.GetOutputBuffer(0, ImportLen);
		WriteUnsignedVarIntWithKnownLength(ImportData.GetData(), BaseIdx + FPacketImport::Bias, BaseIdxLen);
		memcpy((char*)ImportData.GetData() + BaseIdxLen, *Fragment, Fragment.Len());
		ImportWriter.Advance(ImportLen);

		int32 ImportIdx = ImportOffsets.Num() - 1;
		ImportHandles.Add(Handle);
		ImportOffsets.Add(ImportWriter.GetLength());
		ImportIdxPtr = &ImportMap.Add(Handle, ImportIdx);
	}
	return *ImportIdxPtr;
}

void FPacketWriter::WriteTypeTable()
{
	// Write the type table offset in the header
	*(int*)(BufferBasePtr + 8) = (int)Length;

	// Write the types to the end of the buffer
	unsigned char* Span = GetOutputSpanAndAdvance(sizeof(int) + Types.Num() * sizeof(FBlobType));

	*(int*)Span = (int)Types.Num();
	Span += sizeof(int);

	memcpy(Span, Types.GetData(), Types.Num() * sizeof(FBlobType));
}

void FPacketWriter::WriteImportTable()
{
	// Write the the import table offset in the header
	*(int*)(BufferBasePtr + 12) = (int)Length;

	// Allocate the buffers
	int ImportHeaderLength = (ImportOffsets.Num() + 1) * sizeof(int);
	int ImportDataOffset = Length + ImportHeaderLength;
	unsigned char* Span = GetOutputSpanAndAdvance(Align(ImportHeaderLength + ImportWriter.GetLength()));

	// Write the import header
	*(int*)Span = ImportOffsets.Num() - 1;
	Span += sizeof(int);

	for (int ImportOffset : ImportOffsets)
	{
		*(int*)Span = ImportDataOffset + ImportOffset;
		Span += sizeof(int);
	}

	// Write the import data
	ImportWriter.CopyTo(Span);
}

void FPacketWriter::WriteExportTable()
{
	// Write the export table offset in the header
	*(int*)(BufferBasePtr + 16) = (int)Length;

	// Write the exports
	int* Span = (int*)GetOutputSpanAndAdvance((ExportOffsets.Num() + 1) * sizeof(int));
	*(Span++) = (int)(ExportOffsets.Num() - 1);
	memcpy(Span, ExportOffsets.GetData(), ExportOffsets.Num() * sizeof(int));
}

size_t FPacketWriter::Align(size_t Value)
{
	return (Value + 3) & ~3;
}
