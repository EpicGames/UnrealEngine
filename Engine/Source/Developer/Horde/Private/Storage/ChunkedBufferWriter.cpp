// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/ChunkedBufferWriter.h"
#include "../HordePlatform.h"

struct FChunkedBufferWriter::FChunk
{
	FSharedBuffer Buffer;
	size_t Length;

	FChunk(size_t InCapacity)
		: Buffer(FUniqueBuffer::Alloc(InCapacity).MoveToShared())
		, Length(0)
	{
	}

	FChunk(const FChunk&) = delete;

	FChunk(FChunk&& Other)
		: Buffer(MoveTemp(Other.Buffer))
		, Length(Other.Length)
	{
	}

	uint8* GetData()
	{
		return const_cast<uint8*>(static_cast<const uint8*>(Buffer.GetData()));
	}

	const uint8* GetData() const
	{
		return static_cast<const uint8*>(Buffer.GetData());
	}

	FMemoryView GetView() const
	{
		return FMemoryView(Buffer.GetData(), Length);
	}

	FMutableMemoryView GetWritableView()
	{
		return FMutableMemoryView(GetData() + Length, GetCapacity() - Length);
	}

	size_t GetCapacity() const
	{
		return Buffer.GetSize();
	}

	size_t GetLength() const
	{
		return Length;
	}
};

FChunkedBufferWriter::FChunkedBufferWriter(size_t InitialSize)
{
	Chunks.Add(FChunk(InitialSize));
	WrittenLength = 0;
}

FChunkedBufferWriter::~FChunkedBufferWriter()
{
}

void FChunkedBufferWriter::Reset()
{
	WrittenLength = 0;
	Chunks.Empty();
}

size_t FChunkedBufferWriter::GetLength() const
{
	return WrittenLength;
}

FSharedBufferView FChunkedBufferWriter::Slice(size_t Offset, size_t Length) const
{
	size_t EndOffset = Offset + Length;
	if (EndOffset > WrittenLength)
	{
		EndOffset = WrittenLength;
		Length = EndOffset - Offset;
	}

	size_t MinChunkIdx = 0;
	while (MinChunkIdx < Chunks.Num() && Offset > Chunks[MinChunkIdx].GetLength())
	{
		Offset -= Chunks[MinChunkIdx].GetLength();
		MinChunkIdx++;
	}

	size_t MaxChunkIdx = MinChunkIdx;
	while (MaxChunkIdx < Chunks.Num() && EndOffset > Chunks[MaxChunkIdx].GetLength())
	{
		EndOffset -= Chunks[MaxChunkIdx].GetLength();
		MaxChunkIdx++;
	}

	if (MinChunkIdx == MaxChunkIdx)
	{
		const FChunk& Chunk = Chunks[MinChunkIdx];
		return FSharedBufferView(Chunk.Buffer, Offset, Length);
	}

	FUniqueBuffer Buffer = FUniqueBuffer::Alloc(Length);
	{
		size_t SourceOffset = Offset;
		size_t SourceLength = Length;

		uint8* Output = (uint8*)Buffer.GetData();
		for (size_t ChunkIdx = MinChunkIdx; ChunkIdx <= MaxChunkIdx; ChunkIdx++)
		{
			FMemoryView Source = Chunks[ChunkIdx].GetView();
			if (SourceOffset > 0)
			{
				Source = Source.Mid(SourceOffset);
			}
			if (Source.GetSize() > SourceLength)
			{
				Source = Source.Left(SourceLength);
			}

			memcpy(Output, Source.GetData(), Source.GetSize());

			SourceOffset = 0;
			SourceLength -= Source.GetSize();
		}
		check(Output == (const uint8*)Buffer.GetData() + Length);
	}
	return Buffer.MoveToShared();
}

TArray<FMemoryView> FChunkedBufferWriter::GetView() const
{
	TArray<FMemoryView> View;
	for (const FChunk& Chunk : Chunks)
	{
		View.Add(Chunk.GetView());
	}
	return View;
}

void FChunkedBufferWriter::CopyTo(void* Buffer) const
{
	uint8* End = (uint8*)Buffer;
	for (const FChunk& Chunk : Chunks)
	{
		memcpy(End, Chunk.GetData(), Chunk.GetLength());
		End += Chunk.GetLength();
	}
}

FMutableMemoryView FChunkedBufferWriter::GetOutputBuffer(size_t UsedSize, size_t DesiredSize)
{
	FChunk* Chunk = &Chunks.Last();
	if (Chunk->GetLength() + FMath::Max<size_t>(DesiredSize, 1) > Chunk->GetCapacity())
	{
		FChunk NextChunk((DesiredSize + 1024) & ~1023);
		memcpy(NextChunk.GetData(), Chunk->GetData() + Chunk->GetLength(), UsedSize);

		if (Chunk->GetLength() == 0)
		{
			Chunks.Pop();
		}

		Chunks.Add(MoveTemp(NextChunk));
		Chunk = &Chunks.Last();
	}
	return Chunk->GetWritableView();
}

void FChunkedBufferWriter::Advance(size_t Size)
{
	FChunk& Chunk = Chunks.Last();
	check(Chunk.Length + Size <= Chunk.GetCapacity());
	Chunk.Length += Size;
	WrittenLength += Size;
}
