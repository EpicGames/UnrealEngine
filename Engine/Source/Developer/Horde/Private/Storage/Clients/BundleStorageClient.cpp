// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/Clients/BundleStorageClient.h"
#include "Storage/Blob.h"
#include "Storage/Bundles/V2/Packet.h"
#include "Storage/Bundles/V2/BundleWriter.h"
#include "Storage/Bundles/V2/ExportHandle.h"
#include "Storage/Bundles/V2/PacketHandle.h"
#include "HordePlatform.h"

// ---------------------------------------------------------------------------------

class FBundleStorageClient::FBundleHandleData : public FBlobHandleData, public TSharedFromThis<FBundleStorageClient::FBundleHandleData, ESPMode::ThreadSafe>
{
public:
	FBundleStorageClient* StorageClient;
	FBlobHandle Inner;
	mutable bool HasCachedImports;
	mutable TArray<FBlobHandle> CachedImports;

	FBundleHandleData(FBundleStorageClient* InStorageClient, FBlobHandle InInner)
		: StorageClient(InStorageClient)
		, Inner(InInner)
		, HasCachedImports(false)
	{ }

	virtual const char* GetType() const override
	{
		return "FBundleStorageClient::FBundleHandleData";
	}

	virtual bool Equals(const FBlobHandleData& Other) const override
	{
		return Other.GetType() == GetType() && Inner->Equals(*((const FBundleHandleData&)Other).Inner);
	}

	virtual uint32 GetHashCode() const
	{
		return Inner->GetHashCode();
	}

	virtual FBlobHandle GetOuter() const override
	{
		return Inner->GetOuter();
	}

	virtual FBlobType ReadType() const override
	{
		return FBundle::BlobType;
	}

	virtual void ReadImports(TArray<FBlobHandle>& OutImports) const override
	{
		if (!HasCachedImports)
		{
			FBlob BlobData = Read();
			CachedImports = std::move(BlobData.References);
			HasCachedImports = true;
		}
		OutImports = CachedImports;
	}

	virtual FBlob Read() const override
	{
		FSharedBufferView Data = Inner->ReadBody();

		TArray<FBlobLocator> ImportLocators;
		ReadImportsFromData(Data.GetView(), ImportLocators);

		TArray<FBlobHandle> ImportHandles;
		ImportHandles.Reserve(ImportLocators.Num());

		for (const FBlobLocator& ImportLocator : ImportLocators)
		{
			ImportHandles.Add(StorageClient->CreateHandle(ImportLocator));
		}

		return FBlob(FBundle::BlobType, std::move(Data), std::move(ImportHandles));
	}

	virtual FSharedBufferView ReadBody(size_t Offset, TOptional<size_t> Length) const override
	{
		return Inner->ReadBody(Offset, Length);
	}

	static void ReadImportsFromData(FMemoryView Data, TArray<FBlobLocator>& OutLocators)
	{
		FBundleSignature Signature = FBundleSignature::Read(Data.GetData());
		if (Signature.Version > EBundleVersion::LatestV1 && Signature.Version <= EBundleVersion::LatestV2)
		{
			return ReadImportsFromDataV2(Data, OutLocators);
		}
		else
		{
			FHordePlatform::NotSupported("Unsupported bundle version");
		}
	}

	static void ReadImportsFromDataV2(FMemoryView Data, TArray<FBlobLocator>& OutLocators)
	{
		while (Data.GetSize() > 0)
		{
			FBundleSignature Signature = FBundleSignature::Read(Data.GetData());

			FPacket Packet = FPacket::Decode(Data.Left(Signature.Length));
			for (size_t Idx = 0; Idx < Packet.GetImportCount(); Idx++)
			{
				FPacketImport Import = Packet.GetImport(Idx);
				if (Import.GetBaseIdx() == FPacketImport::InvalidBaseIdx)
				{
					OutLocators.Add(FBlobLocator(Import.GetFragment()));
				}
			}

			Data = Data.Mid(Signature.Length);
		}
	}

	virtual bool TryAppendIdentifier(FUtf8String& OutBuffer) const override
	{
		return Inner->TryAppendIdentifier(OutBuffer);
	}

	virtual FBlobHandle GetFragmentHandle(const FUtf8StringView& Fragment) const override
	{
		TBlobHandle<FBundleHandleData> BundleHandle = const_cast<FBundleStorageClient::FBundleHandleData*>(this)->AsShared();
		return StorageClient->GetFragmentHandle(BundleHandle, Fragment);
	}
};

FBundleStorageClient::FBundleStorageClient(TSharedRef<FKeyValueStorageClient> InInner)
	: Inner(MoveTemp(InInner))
{ 
}

FBundleStorageClient::~FBundleStorageClient()
{
}

FBlob FBundleStorageClient::ReadBlob(const FBlobLocator& Locator) const
{
	return Inner->ReadBlob(Locator);
}

FBlobHandle FBundleStorageClient::WriteBlob(const FUtf8StringView& BasePath, const FBlobType& Type, const TArrayView<FMemoryView>& Data, const TArrayView<FBlobHandle>& Imports)
{
	return Inner->WriteBlob(BasePath, Type, Data, Imports);
}

FBlobHandle FBundleStorageClient::CreateHandle(const FBlobLocator& Locator) const
{
	FBlobLocator BaseLocator;
	FUtf8StringView Fragment;

	if (Locator.TryUnwrap(BaseLocator, Fragment))
	{
		return GetFragmentHandle(CreateBundleHandle(BaseLocator), Fragment);
	}
	else
	{
		return CreateBundleHandle(Locator);
	}
}

TUniquePtr<FBlobWriter> FBundleStorageClient::CreateWriter(FUtf8String BasePath)
{
	return CreateWriter(BasePath, FBundleOptions::Default);
}

TUniquePtr<FBlobWriter> FBundleStorageClient::CreateWriter(FUtf8String BasePath, const FBundleOptions& Options)
{
	if (Options.MaxVersion == EBundleVersion::LatestV2)
	{
		return TUniquePtr<FBlobWriter>(new FBundleWriter(StaticCastSharedRef<FKeyValueStorageClient>(AsShared()), BasePath, Options));
	}
	else
	{
		FHordePlatform::NotSupported("Unsupported bundle version");
	}
}

void FBundleStorageClient::AddAlias(const char* Name, FBlobAlias Alias)
{
	Inner->AddAlias(Name, MoveTemp(Alias));
}

void FBundleStorageClient::RemoveAlias(const char* Name, FBlobHandle Handle)
{
	Inner->RemoveAlias(Name, MoveTemp(Handle));
}

void FBundleStorageClient::FindAliases(const char* Name, int MaxResults, TArray<FBlobAlias>& OutAliases)
{
	OutAliases.Empty();
	Inner->FindAliases(Name, MaxResults, OutAliases);

	for (FBlobAlias& Alias : OutAliases)
	{
		Alias.Target = CreateHandle(Alias.Target->GetLocator());
	}
}

bool FBundleStorageClient::DeleteRef(const FRefName& Name)
{
	return Inner->DeleteRef(Name);
}

FBlobHandle FBundleStorageClient::ReadRef(const FRefName& Name, const FRefCacheTime& CacheTime) const
{
	FBlobHandle Target = Inner->ReadRef(Name, CacheTime);
	if (Target)
	{
		Target = CreateHandle(Target->GetLocator());
	}
	return Target;
}

void FBundleStorageClient::WriteRef(const FRefName& Name, const FBlobHandle& Handle, const FRefOptions& Options)
{
	Inner->WriteRef(Name, Handle, Options);
}

TBlobHandle<FBundleStorageClient::FBundleHandleData> FBundleStorageClient::CreateBundleHandle(const FBlobLocator& Locator) const
{
	FBlobHandle Handle = Inner->CreateHandle(Locator);
	return MakeShared<FBundleHandleData>(const_cast<FBundleStorageClient*>(this), Handle);
}

FBlobHandle FBundleStorageClient::GetFragmentHandle(const FBlobHandle& BundleHandle, const FUtf8StringView& Fragment) const
{
	int32 AmpIdx;
	if (Fragment.FindChar((UTF8CHAR)'&', AmpIdx))
	{
		TBlobHandle<FPacketHandleData> PacketHandle = MakeShared<FPacketHandleData>(const_cast<FStorageClient*>(static_cast<const FStorageClient*>(this))->AsShared(), BundleHandle, Fragment.Mid(0, AmpIdx));
		return MakeShared<FExportHandleData>(PacketHandle, Fragment.Mid(AmpIdx + 1));
	}
	else
	{
		return MakeShared<FPacketHandleData>(const_cast<FStorageClient*>(static_cast<const FStorageClient*>(this))->AsShared(), BundleHandle, Fragment);
	}
}
