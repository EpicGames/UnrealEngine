// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/Clients/FileStorageClient.h"
#include "Storage/Blob.h"
#include "Storage/BlobType.h"
#include <iostream>
#include <sstream>
#include <fstream>
#include "HordePlatform.h"

FFileStorageClient::FFileStorageClient(std::filesystem::path InRootDir)
	: RootDir(std::move(InRootDir))
{
}

FFileStorageClient::~FFileStorageClient()
{
}

FBlobLocator FFileStorageClient::ReadRefFromFile(const std::filesystem::path& File)
{
	std::ifstream Stream(File);

	std::stringstream Buffer;
	Buffer << Stream.rdbuf();

	return FBlobLocator(FUtf8String(Buffer.str().c_str()));
}

void FFileStorageClient::WriteRefToFile(const std::filesystem::path& File, const FBlobLocator& Locator)
{
	std::ofstream Stream(File);

	const FUtf8String& Path = Locator.GetPath();
	Stream.write((const char*)*Path, Path.Len());
}

std::filesystem::path FFileStorageClient::GetBlobFile(const FBlobLocator& Locator) const
{
	return RootDir / (std::string((const char*)*Locator.GetPath()) + ".blob");
}

std::filesystem::path FFileStorageClient::GetRefFile(const FRefName& Name) const
{
	return RootDir / (std::string((const char*)*Name.GetText()) + ".ref");
}

void FFileStorageClient::AddAlias(const char* Name, FBlobAlias Alias)
{
	FHordePlatform::NotSupported("File storage client does not currently support aliases.");
}

void FFileStorageClient::RemoveAlias(const char* Name, FBlobHandle Handle)
{
	FHordePlatform::NotSupported("File storage client does not currently support aliases.");
}

void FFileStorageClient::FindAliases(const char* Name, int MaxResults, TArray<FBlobAlias>& OutAliases)
{
	FHordePlatform::NotSupported("File storage client does not currently support aliases.");
}

bool FFileStorageClient::DeleteRef(const FRefName& Name)
{
	std::filesystem::path File = GetRefFile(Name);
	if (std::filesystem::is_regular_file(std::filesystem::status(File)))
	{
		std::filesystem::remove(File);
		return true;
	}
	return false;
}

FBlobHandle FFileStorageClient::ReadRef(const FRefName& Name, const FRefCacheTime& CacheTime) const
{
	std::filesystem::path File = GetRefFile(Name);
	if (std::filesystem::is_regular_file(std::filesystem::status(File)))
	{
		return FBlobHandle();
	}

	return CreateHandle(ReadRefFromFile(File));
}

void FFileStorageClient::WriteRef(const FRefName& Name, const FBlobHandle& Target, const FRefOptions& Options)
{
	FBlobLocator Locator = Target->GetLocator();

	std::filesystem::path File = GetRefFile(Name);
	std::filesystem::create_directory(File.parent_path());

	std::ofstream Stream(File);
	Stream << std::string((const char*)*Locator.GetPath());
}

FBlob FFileStorageClient::ReadBlob(const FBlobLocator& Locator) const
{
	std::filesystem::path FileName = GetBlobFile(Locator);

	std::ifstream Stream(FileName, std::ios::binary | std::ios::ate);
	check(Stream);

	std::streampos Length = Stream.tellg();
	check(Length != std::streampos(-1));
	Stream.seekg(0, std::ios::beg);

	FUniqueBuffer Buffer = FUniqueBuffer::Alloc(Length);
	Stream.read((char*)Buffer.GetData(), Length);

	return FBlob(FBlobType::Leaf, Buffer.MoveToShared(), TArray<FBlobHandle>());
}

FBlobHandle FFileStorageClient::WriteBlob(const FUtf8StringView& BasePath, const FBlobType& Type, const TArrayView<FMemoryView>& Data, const TArrayView<FBlobHandle>& Imports)
{
	FBlobLocator Locator = CreateLocator(BasePath);

	std::filesystem::path FileName = GetBlobFile(Locator);
	std::filesystem::create_directories(FileName.parent_path());

	std::ofstream Stream(FileName, std::ios::binary);
	for (FMemoryView& View : Data)
	{
		Stream.write((const char*)View.GetData(), View.GetSize());
	}
	Stream.close();

	return CreateHandle(Locator);
}

