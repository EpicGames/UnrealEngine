// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/Clients/KeyValueStorageClient.h"
#include "Storage/Blob.h"
#include "HordePlatform.h"

class FKeyValueStorageClient::FHandleData : public FBlobHandleData
{
public:
	FKeyValueStorageClient& StorageClient;
	const FBlobLocator Locator;

	FHandleData(FKeyValueStorageClient& InStorageClient, const FBlobLocator& InLocator)
		: StorageClient(InStorageClient)
		, Locator(InLocator)
	{
		check(Locator.GetPath().Len() > 0);
	}

	virtual const char* GetType() const override
	{
		return "FKeyValueStorageClient::FHandleData";
	}

	virtual uint32 GetHashCode() const override
	{
		FHordePlatform::NotImplemented();
//		return std::hash<std::string>()(Locator.GetPath());
	}

	virtual bool Equals(const FBlobHandleData& Other) const override
	{
		return Other.GetType() == GetType() && ((const FHandleData&)Other).Locator == Locator;
	}

	virtual FBlobHandle GetOuter() const override
	{
		return FBlobHandle();
	}

	virtual FBlob Read() const override
	{
		return StorageClient.ReadBlob(Locator);
	}

	virtual bool TryAppendIdentifier(FUtf8String& OutIdentifier) const override
	{
		OutIdentifier += Locator.GetPath();
		return true;
	}
};

FBlobHandle FKeyValueStorageClient::CreateHandle(const FBlobLocator& Locator) const
{
	FBlobLocator BaseLocator;
	FUtf8StringView Fragment;

	if (Locator.TryUnwrap(BaseLocator, Fragment))
	{
		FHordePlatform::NotImplemented();
//		return new FBlobFragmentHandle(new Handle(this, baseLocator), fragment);
	}
	else
	{
		return MakeShared<FHandleData>(const_cast<FKeyValueStorageClient&>(*this), Locator);
	}
}

TUniquePtr<FBlobWriter> FKeyValueStorageClient::CreateWriter(FUtf8String BasePath)
{
	FHordePlatform::NotImplemented();
}

