// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/Clients/MemoryStorageClient.h"
#include "Storage/Clients/KeyValueStorageClient.h"
#include "Storage/Blob.h"

FMemoryStorageClient::FMemoryStorageClient()
{
}

FMemoryStorageClient::~FMemoryStorageClient()
{
}

void FMemoryStorageClient::AddAlias(const char* Name, FBlobAlias Alias)
{
	FBlobLocator Locator = Alias.Target->GetLocator();

	FScopeLock Lock(&CriticalSection);

	TArray<FBlobAlias>& AliasList = Aliases.Emplace(FUtf8String(Name), TArray<FBlobAlias>());
	AliasList.Add(MoveTemp(Alias));
}

void FMemoryStorageClient::RemoveAlias(const char* Name, FBlobHandle Handle)
{
	FScopeLock Lock(&CriticalSection);

	TArray<FBlobAlias>* AliasList = Aliases.Find(Name);
	if (AliasList == nullptr)
	{
		return;
	}

	FBlobLocator locator = Handle->GetLocator();
	AliasList->RemoveAll([&Handle](const FBlobAlias& Alias) -> bool { return Alias.Target->Equals(*Handle); });

	if (AliasList->Num() == 0)
	{
		Aliases.Remove(Name);
	}
}
	
void FMemoryStorageClient::FindAliases(const char* Name, int MaxResults, TArray<FBlobAlias>& OutAliases)
{
	FScopeLock Lock(&CriticalSection);

	OutAliases.Empty();

	const TArray<FBlobAlias>* Result = Aliases.Find(Name);
	if (Result != nullptr)
	{
		OutAliases = *Result;
	}
}

bool FMemoryStorageClient::DeleteRef(const FRefName& Name)
{
	return Refs.Remove(Name) > 0;
}

FBlobHandle FMemoryStorageClient::ReadRef(const FRefName& Name, const FRefCacheTime& CacheTime) const
{
	FScopeLock Lock(&CriticalSection);

	const FBlobLocator* Locator = Refs.Find(Name);
	if (Locator == nullptr)
	{
		return FBlobHandle();
	}
	else
	{
		return CreateHandle(*Locator);
	}
}

void FMemoryStorageClient::WriteRef(const FRefName& Name, const FBlobHandle& Target, const FRefOptions& Options)
{
	FScopeLock Lock(&CriticalSection);
	Refs[Name] = Target->GetLocator();
}

FBlob FMemoryStorageClient::ReadBlob(const FBlobLocator& Locator) const
{
	FScopeLock Lock(&CriticalSection);
	return Blobs.FindChecked(Locator);
}

FBlobHandle FMemoryStorageClient::WriteBlob(const FUtf8StringView& BasePath, const FBlobType& Type, const TArrayView<FMemoryView>& Data, const TArrayView<FBlobHandle>& Imports)
{
	size_t Length = 0;
	for (const FMemoryView& View : Data)
	{
		Length += View.GetSize();
	}

	FUniqueBuffer Buffer = FUniqueBuffer::Alloc(Length);

	uint8* Output = (uint8*)Buffer.GetData();
	for (const FMemoryView& View : Data)
	{
		memcpy(Output, View.GetData(), View.GetSize());
		Output += View.GetSize();
	}

	FScopeLock Lock(&CriticalSection);

	FBlobLocator Locator = CreateLocator(BasePath);

	FBlob Blob(Type, Buffer.MoveToShared(), TArray<FBlobHandle>(Imports));
	Blobs.Emplace(Locator, MoveTemp(Blob));

	return CreateHandle(Locator);
}
