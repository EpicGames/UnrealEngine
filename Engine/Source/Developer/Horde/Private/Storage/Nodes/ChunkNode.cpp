// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/Nodes/ChunkNode.h"
#include "Storage/Blob.h"

const FChunkingOptions FChunkingOptions::Default{};

// ----------------------------------------------------------------------

const FBlobType FChunkNode::LeafBlobType(FGuid(0xB27AFB68, 0x4A4B9E20, 0x8A78D8A4, 0x39D49840), 1);
const FBlobType FChunkNode::InteriorBlobType(FGuid(0xF4DEDDBC, 0x4C7A70CB, 0x11F04783, 0xB9CDCCAF), 2);

FChunkNode::FChunkNode()
{
}

FChunkNode::FChunkNode(TArray<FBlobHandleWithHash> InChildren, FSharedBufferView InData)
	: Children(MoveTemp(InChildren))
	, Data(MoveTemp(InData))
{
}

FChunkNode::~FChunkNode()
{
}

FChunkNode FChunkNode::Read(FBlob Blob)
{
	FChunkNode Node;

	const uint8* InputData = Blob.Data.GetPointer();
	for (int32 Idx = 0; Idx < Blob.References.Num(); Idx++)
	{
		FIoHash Hash;
		memcpy(&Hash, InputData, sizeof(FIoHash));
		InputData += sizeof(FIoHash) + 1; // Ignore node type

		Node.Children.Add(FBlobHandleWithHash(MoveTemp(Blob.References[Idx]), Hash));
	}

	Node.Data = Blob.Data.Slice(sizeof(FIoHash) * Blob.References.Num());
	return MoveTemp(Node);
}

FBlobHandleWithHash FChunkNode::Write(FBlobWriter& Writer) const
{
	return Write(Writer, Children, Data.GetView());
}

FBlobHandleWithHash FChunkNode::Write(FBlobWriter& Writer, const TArrayView<const FBlobHandleWithHash>& Children, FMemoryView Data)
{
	int32 BufferSize = ((sizeof(FIoHash) + 1) * Children.Num()) + Data.GetSize();
	uint8* Buffer = (uint8*)Writer.GetOutputBuffer(BufferSize);

	uint8* NextData = (uint8*)Buffer;
	for (int32 Idx = 0; Idx < Children.Num(); Idx++)
	{
		memcpy(NextData, &Children[Idx].Hash, sizeof(FIoHash));
		NextData += sizeof(FIoHash);

		*NextData = 1; // Leaf node
		NextData++;

		Writer.AddImport(Children[Idx].Handle);
	}
	memcpy(NextData, Data.GetData(), Data.GetSize());

	FIoHash Hash = FIoHash::HashBuffer(Buffer, BufferSize);
	Writer.Advance(BufferSize);

	FBlobHandle Handle = Writer.CompleteBlob((Children.Num() == 0)? LeafBlobType : InteriorBlobType);
	return FBlobHandleWithHash(MoveTemp(Handle), Hash);
}

// ----------------------------------------------------------------------

FChunkNodeReader::FChunkNodeReader(FBlob Blob)
{
	Stack.Add(FStackEntry(MoveTemp(Blob)));
	Advance(0);
}

FChunkNodeReader::FChunkNodeReader(const FBlobHandle& Handle)
	: FChunkNodeReader(Handle->Read())
{
}

FChunkNodeReader::~FChunkNodeReader()
{
}

bool FChunkNodeReader::IsComplete() const
{
	return Stack.Num() == 0;
}

FMemoryView FChunkNodeReader::GetBuffer() const
{
	if (Stack.Num() == 0)
	{
		return FMemoryView();
	}

	const FStackEntry& StackTop = Stack.Top();
	FMemoryView View = StackTop.Blob.Data.GetView();
	return View.Mid(StackTop.Position);
}

void FChunkNodeReader::Advance(int32 Size)
{
	while (Stack.Num() > 0)
	{
		FStackEntry& StackTop = Stack.Top();

		FMemoryView Data = StackTop.Blob.Data.GetView();
		if (StackTop.Position == Data.GetSize())
		{
			Stack.Pop();
		}
		else if (StackTop.Position < sizeof(FIoHash) * StackTop.Blob.References.Num())
		{
			FBlobHandle ChildHandle = StackTop.Blob.References[StackTop.Position / sizeof(FIoHash)];
			StackTop.Position += sizeof(FIoHash);
			Stack.Add(FStackEntry(ChildHandle->Read()));
		}
		else if (Size > 0)
		{
			size_t AdvanceSize = FMath::Min<size_t>(Size, Data.GetSize() - StackTop.Position);
			StackTop.Position += AdvanceSize;
			Size -= AdvanceSize;
		}
		else
		{
			break;
		}
	}
}

FChunkNodeReader::operator bool() const
{
	return !IsComplete();
}

// ----------------------------------------------------------------------

FChunkNodeWriter::FChunkNodeWriter(FBlobWriter& InWriter, const FChunkingOptions& InOptions)
	: Writer(InWriter)
	, Options(InOptions)
	, Threshold((1LL << 32) / Options.TargetChunkSize)
	, NodeLength(0)
{
}

FChunkNodeWriter::~FChunkNodeWriter()
{
}

void FChunkNodeWriter::Write(FMemoryView Data)
{
	StreamHasher.Update(Data);

	while (Data.GetSize() > 0)
	{
		uint8* NodeBuffer = (uint8*)Writer.GetOutputBuffer(Options.MaxChunkSize);

		// Add up to the minimum chunk size
		if (NodeLength < Options.MinChunkSize)
		{
			uint64 AppendLength = FMath::Min<uint64>(Options.MinChunkSize - NodeLength, Data.GetSize());
			memcpy(NodeBuffer + NodeLength, Data.GetData(), AppendLength);
			NodeLength += AppendLength;
			Data = Data.Mid(AppendLength);
		}

		const uint8* NextSpan = (const uint8*)Data.GetData();

		// Step forwards until reaching the threshold
		int32 Idx = 0;
		while (RollingHash.Get() > Threshold)
		{
			if (Idx == Data.GetSize())
			{
				memcpy(NodeBuffer + NodeLength, Data.GetData(), Data.GetSize());
				NodeLength += Data.GetSize();
				return;
			}

			NodeBuffer[NodeLength] = NextSpan[Idx];
			RollingHash.Add(NextSpan[Idx]);
			RollingHash.Sub(NodeBuffer[NodeLength - Options.MinChunkSize]);

			NodeLength++;
			Idx++;
		}

		// Write the current node
		WriteNode();
	}
}

FBlobHandleWithHash FChunkNodeWriter::Flush(FIoHash& OutStreamHash)
{
	// Finish writing the current node
	if (NodeLength > 0 || Nodes.Num() == 0)
	{
		WriteNode();
	}

	// Get the hash for the whole stream
	OutStreamHash = FIoHash(StreamHasher.Finalize());

	// If there was only one node, we don't need an interior node on top of it
	if (Nodes.Num() == 1)
	{
		return Nodes[0];
	}

	// Create the interior node
	return FChunkNode::Write(Writer, Nodes, FMemoryView());
}

void FChunkNodeWriter::WriteNode()
{
	// Update the writer with the length of the written data
	uint8* NodeBuffer = (uint8*)Writer.GetOutputBuffer(NodeLength);
	FIoHash NodeHash = FIoHash::HashBuffer(NodeBuffer, NodeLength);
	Writer.Advance(NodeLength);

	// Create the blob handle
	FBlobHandle NodeHandle = Writer.CompleteBlob(FChunkNode::LeafBlobType);
	Nodes.Add(FBlobHandleWithHash(MoveTemp(NodeHandle), NodeHash));

	// Clear the current state
	NodeLength = 0;
	RollingHash.Reset();
}
