// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/Nodes/DirectoryEntry.h"
#include "Storage/BlobReader.h"
#include "Storage/BlobWriter.h"

FDirectoryEntry::FDirectoryEntry(FBlobHandle InTarget, const FIoHash& InTargetHash, FUtf8String InName, int64 InLength)
	: Target(MoveTemp(InTarget))
	, TargetHash(InTargetHash)
	, Length(InLength)
	, Name(MoveTemp(InName))
{
}

FDirectoryEntry::~FDirectoryEntry()
{
}

FDirectoryEntry FDirectoryEntry::Read(FBlobReader& Reader)
{
	FBlobHandle Target = Reader.ReadImport();
	FIoHash TargetHash = ReadIoHash(Reader);

	int64 Length = (int64)ReadUnsignedVarInt(Reader);
	FUtf8String Name = ReadString(Reader);

	return FDirectoryEntry(MoveTemp(Target), TargetHash, MoveTemp(Name), Length);
}

void FDirectoryEntry::Write(FBlobWriter& Writer) const
{
	Writer.AddImport(Target);
	WriteIoHash(Writer, TargetHash);

	WriteUnsignedVarInt(Writer, Length);
	WriteString(Writer, Name);
}

