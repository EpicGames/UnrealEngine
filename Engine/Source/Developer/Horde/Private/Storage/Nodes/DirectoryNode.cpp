// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/Nodes/DirectoryNode.h"
#include "Storage/Blob.h"
#include "Storage/BlobReader.h"
#include "Storage/BlobWriter.h"
#include "HordePlatform.h"

const FBlobType FDirectoryNode::BlobType(FGuid(0x0714EC11, 0x4D07291A, 0x8AE77F86, 0x799980D6), 1);

FDirectoryNode::FDirectoryNode(EDirectoryFlags InFlags)
	: Flags(InFlags)
{
}

FDirectoryNode::~FDirectoryNode()
{
}

FDirectoryNode FDirectoryNode::Read(const FBlob& Blob)
{
	FBlobReader Reader(Blob);

	EDirectoryFlags Flags = (EDirectoryFlags)ReadUnsignedVarInt(Reader);
	FDirectoryNode Directory(Flags);

	int32 FileCount = (int32)ReadUnsignedVarInt(Reader);
	for (int32 Idx = 0; Idx < FileCount; Idx++)
	{
		FFileEntry Entry = FFileEntry::Read(Reader);
		FUtf8String Name = Entry.Name;
		Directory.NameToFile.Emplace(MoveTemp(Name), MoveTemp(Entry));
	}

	int32 DirectoryCount = (int32)ReadUnsignedVarInt(Reader);
	for (int32 Idx = 0; Idx < DirectoryCount; Idx++)
	{
		FDirectoryEntry Entry = FDirectoryEntry::Read(Reader);
		FUtf8String Name = Entry.Name;
		Directory.NameToDirectory.Emplace(MoveTemp(Name), MoveTemp(Entry));
	}

	return Directory;
}

FBlobHandle FDirectoryNode::Write(FBlobWriter& Writer) const
{
	WriteUnsignedVarInt(Writer, (int64)Flags);

	WriteUnsignedVarInt(Writer, NameToFile.Num());
	for(TMap<FUtf8String, FFileEntry>::TConstIterator Iter(NameToFile); Iter; ++Iter)
	{
		Iter.Value().Write(Writer);
	}

	WriteUnsignedVarInt(Writer, NameToDirectory.Num());
	for (TMap<FUtf8String, FDirectoryEntry>::TConstIterator Iter(NameToDirectory); Iter; ++Iter)
	{
		Iter.Value().Write(Writer);
	}

	return Writer.CompleteBlob(BlobType);
}

