// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/Nodes/FileEntry.h"
#include "IO/IoHash.h"
#include "Storage/BlobReader.h"
#include "Storage/BlobWriter.h"
#include "HordePlatform.h"

FFileEntry::FFileEntry(FBlobHandleWithHash InTarget, FUtf8String InName, EFileEntryFlags InFlags, int64 InLength, const FIoHash& InHash, FSharedBufferView InCustomData)
	: Target(MoveTemp(InTarget))
	, Name(MoveTemp(InName))
	, Flags(InFlags)
	, Length(InLength)
	, Hash(InHash)
	, CustomData(MoveTemp(InCustomData))
{ }

FFileEntry::~FFileEntry()
{ }

FFileEntry FFileEntry::Read(FBlobReader& Reader)
{
	FBlobHandleWithHash Target = ReadBlobHandleWithHash(Reader);
	if (Reader.GetVersion() >= 2)
	{
		ReadUnsignedVarInt(Reader);
	}

	FUtf8String Name = ReadString(Reader);
	EFileEntryFlags Flags = (EFileEntryFlags)ReadUnsignedVarInt(Reader);
	int64 Length = (int64)ReadUnsignedVarInt(Reader);
	FIoHash Hash = ReadIoHash(Reader);

	FSharedBufferView CustomData;
	if (EnumHasAnyFlags(Flags, EFileEntryFlags::HasCustomData))
	{
		FHordePlatform::NotImplemented();
//		CustomData = ReadVariableLengthBytes(CustomData);
//		Flags &= ~EFileEntryFlags::HasCustomData;
	}

	return FFileEntry(MoveTemp(Target), MoveTemp(Name), Flags, Length, Hash, MoveTemp(CustomData));
}

void FFileEntry::Write(FBlobWriter& Writer) const
{
	WriteBlobHandleWithHash(Writer, Target);

	EFileEntryFlags WriteFlags = Flags; // TODO: (CustomData.Length > 0) ? (Flags | EFileEntryFlags::HasCustomData) : (Flags & ~EFileEntryFlags::HasCustomData);

	WriteString(Writer, Name);
	WriteUnsignedVarInt(Writer, (int)WriteFlags);
	WriteUnsignedVarInt(Writer, Length);
	WriteIoHash(Writer, Hash);

	if (EnumHasAnyFlags(WriteFlags, EFileEntryFlags::HasCustomData))
	{
		FHordePlatform::NotImplemented();
//		writer.WriteVariableLengthBytes(CustomData.Span);
	}
}

