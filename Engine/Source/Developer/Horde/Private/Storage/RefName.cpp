// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/RefName.h"

FRefName::FRefName(FUtf8String InText)
	: Text(MoveTemp(InText))
{
}
FRefName::~FRefName()
{
}

const FUtf8String& FRefName::GetText() const
{
	return Text;
}

bool FRefName::operator==(const FRefName& Other) const
{
	return Text.Equals(Other.Text, ESearchCase::CaseSensitive);
}

bool FRefName::operator!=(const FRefName& Other) const
{
	return !(*this == Other);
}

uint32 GetTypeHash(const FRefName& RefName)
{
	return GetTypeHash(RefName.Text);
}
