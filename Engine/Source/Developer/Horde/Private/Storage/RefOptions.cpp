// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/RefOptions.h"

const FRefOptions FRefOptions::Default;

FRefOptions::FRefOptions()
{
}

