// Copyright Epic Games, Inc. All Rights Reserved.

#include "Storage/StorageClient.h"
#include "Storage/RefOptions.h"
#include "../HordePlatform.h"

FRefCacheTime::FRefCacheTime()
{
}

FRefCacheTime::FRefCacheTime(FTimespan InMaxAge)
	: Time(FDateTime::UtcNow() - InMaxAge)
{
}

FRefCacheTime::FRefCacheTime(FDateTime InTime)
	: Time(InTime)
{
}

// --------------------------------------------------------------------------------------------------------

FStorageClient::~FStorageClient()
{
}

void FStorageClient::WriteRef(const FRefName& Name, const FBlobHandle& Handle)
{
	WriteRef(Name, Handle, FRefOptions::Default);
}

FBlobLocator FStorageClient::CreateLocator(const FUtf8StringView& BasePath)
{
	FUtf8String Path(BasePath);
	if (Path.Len() > 0 && !Path.EndsWith("/", ESearchCase::CaseSensitive))
	{
		Path += "/";
	}

	char Name[128];
	FHordePlatform::CreateUniqueIdentifier(Name, sizeof(Name) / sizeof(Name[0]));
	Path += Name;

	return FBlobLocator(MoveTemp(Path));
}

// --------------------------------------------------------------------------------------------------------

FBlobAlias::FBlobAlias(FBlobHandle InTarget, int InRank, FSharedBufferView InData)
	: Target(MoveTemp(InTarget))
	, Rank(InRank)
	, Data(std::move(InData))
{
}

FBlobAlias::~FBlobAlias()
{
}

// --------------------------------------------------------------------------------------------------------

FAliasInfo::FAliasInfo(FUtf8String InName, int InRank)
	: Name(MoveTemp(InName))
	, Rank(InRank)
{
}

FAliasInfo::FAliasInfo(FUtf8String InName, int InRank, FSharedBufferView InData)
	: Name(MoveTemp(InName))
	, Rank(InRank)
	, Data(MoveTemp(InData))
{
}

FAliasInfo::~FAliasInfo()
{
}

