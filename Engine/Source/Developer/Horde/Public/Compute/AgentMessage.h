// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include <memory>
#include <vector>
#include <map>
#include <string_view>
#include "ComputeBuffer.h"
#include "ComputeChannel.h"

// Type of a compute message
enum class EAgentMessageType : unsigned char
{
	// No message was received (end of stream)
	None = 0x00,

	// No-op message sent to keep the connection alive. Remote should reply with the same message.
	Ping = 0x01,

	// Sent in place of a regular response if an error occurs on the remote
	Exception = 0x02,

	// Fork the message loop into a new channel
	Fork = 0x03,

	// Sent as the first message on a channel to notify the remote that the remote end is attached
	Attach = 0x04,

	// Extract files on the remote machine (Initiator -> Remote)
	WriteFiles = 0x10,

	// Notification that files have been extracted (Remote -> Initiator)
	WriteFilesResponse = 0x11,

	// Deletes files on the remote machine (Initiator -> Remote)
	DeleteFiles = 0x12,

	// Execute a process in a sandbox (Initiator -> Remote)
	ExecuteV1 = 0x16,

	// Execute a process in a sandbox (Initiator -> Remote)
	ExecuteV2 = 0x22,

	// Returns output from the child process to the caller (Remote -> Initiator)
	ExecuteOutput = 0x17,

	// Returns the process exit code (Remote -> Initiator)
	ExecuteResult = 0x18,

	// Reads a blob from storage
	ReadBlob = 0x20,

	// Response to a ReadBlob request.
	ReadBlobResponse = 0x21,

	// Xor a block of data with a value
	XorRequest = 0xf0,

	// Result from an XorRequest request.
	XorResponse = 0xf1,
};

// Flags describing how to execute a compute task process on the agent
enum class EExecuteProcessFlags : unsigned char
{
	// No execute flags set
	None = 0,

	// Request execution to be wrapped under Wine when running on Linux. Agent still reserves the right to refuse it (e.g no Wine executable configured, mismatching OS etc)
	UseWine = 1,
};

// Definitions for various message types
namespace AgentMessage
{
	struct FException
	{
		FUtf8StringView Message;
		FUtf8StringView Description;
	};

	struct FBlobRequest
	{
		FUtf8StringView Locator;
		int Offset;
		int Length;
	};
}

// Channel for receiving agent messages
class FAgentMessageChannel
{
public:
	HORDE_API FAgentMessageChannel(TSharedPtr<FComputeChannel> InChannel);
	HORDE_API ~FAgentMessageChannel();

	//// Requests ////

	// Closes the remote message loop
	HORDE_API void Close();

	// Sends a ping message to the remote
	HORDE_API void Ping();

	// Sends an exception response to the remote
	HORDE_API void Exception(const char* Description, const char* Trace);

	// Requests that the remote message loop be forked
	HORDE_API void Fork(int ChannelId, int BufferSize);

	// Notifies the remote that a buffer has been attached
	HORDE_API void Attach();

	// Extracts a bundle containing files to a particular path
	HORDE_API void UploadFiles(const char* Path, const char* Locator);

	// Deletes files matching a set of wildcards
	HORDE_API void DeleteFiles(const char* const * Paths, size_t Count);

	// Executes a process on the remote machine
	HORDE_API void Execute(const char* Exe, const char* const * Args, size_t NumArgs, const char* WorkingDir, const char* const * EnvVars, size_t NumEnvVars, EExecuteProcessFlags Flags);

	// Writes a blob requested by the remote
	HORDE_API void Blob(const unsigned char* Data, size_t Length);

	// Send a message to request that a byte string be xor'ed with a particular value
	HORDE_API void Xor(const unsigned char* Data, size_t Length, unsigned char Value);

	//// Responses ////

	// Reads a response from the remote. Other Read methods can be used to access response data.
	HORDE_API EAgentMessageType ReadResponse(int32 TimeoutMS = -1, bool* bOutTimedOut = nullptr);

	// Gets the raw data from a response
	const void* GetResponseData() const { return ResponseData; }

	// Gets the size of the response
	size_t GetResponseSize() const { return ResponseLength; }

	// Reads an exception response
	HORDE_API void ReadException(AgentMessage::FException& Ex);

	// Reads the result from executing a process
	HORDE_API int ReadExecuteResult();

	// Reads a blob request message
	HORDE_API void ReadBlobRequest(AgentMessage::FBlobRequest& Ex);

private:
	const size_t MessageHeaderLength = 5;

	TSharedPtr<FComputeChannel> ChannelBuffers;

	unsigned char* RequestData;
	size_t RequestSize;
	size_t MaxRequestSize;

	EAgentMessageType ResponseType;
	const unsigned char* ResponseData;
	size_t ResponseLength;

	void CreateMessage(EAgentMessageType Type, size_t MaxLength);
	void FlushMessage();

	void WriteInt32(int Value);
	static int ReadInt32(const unsigned char** Pos);

	void WriteFixedLengthBytes(const unsigned char* Data, size_t Length);
	static const unsigned char* ReadFixedLengthBytes(const unsigned char** Pos, size_t Length);

	static size_t MeasureUnsignedVarInt(size_t Value);
	void WriteUnsignedVarInt(size_t Value);
	static size_t ReadUnsignedVarInt(const unsigned char** Pos);

	size_t MeasureString(const char* Text) const;
	void WriteString(const char* Text);
	void WriteString(const std::string_view& Text);
	static FUtf8StringView ReadString(const unsigned char** Pos);

	void WriteOptionalString(const char* Text);
};

