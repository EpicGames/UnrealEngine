// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include <memory>

struct FHeader;
struct FComputeBufferDetail;

class FComputeBufferReader;
class FComputeBufferWriter;

//
// Implements a ring buffer using shared memory, with one writer and multiple readers. 
// 
// Contention between the writer and readers is resolved by splitting the buffer into a number of
// fixed-size chunks, which can be appended to and read immediately, but only reset once consumed
// by all readers.
//
// If the read position is never advanced and only one chunk is used, the buffer functions as an
// append-only buffer.
//
class FComputeBuffer
{
public:
	// Maximum allowed number of readers
	static const int MaxReaders = 16;

	// Maximum allowed number of chunks
	static const int MaxChunks = 16;

	// Maximum length of a compute buffer name
	static const size_t MaxNameLength = 256;

	// Parameters for constructing a compute buffer
	struct FParams
	{
		const char* Name = nullptr; // When creating a buffer, we should not normally set the name. We can only *OPEN* buffers with a name. Usually we create buffers and attach them to channels.
		int NumChunks = 2;
		int ChunkLength = 512 * 1024;
		int NumReaders = 1;
		bool Anonymous = false; // Set this to true to use anonymous buffers. This could be used together with for example socket transports
	};

	HORDE_API FComputeBuffer();
	HORDE_API FComputeBuffer(const FComputeBuffer& Other);
	HORDE_API FComputeBuffer(FComputeBuffer&& Other) noexcept;
	HORDE_API ~FComputeBuffer();

	HORDE_API FComputeBuffer& operator=(const FComputeBuffer& Other);
	HORDE_API FComputeBuffer& operator=(FComputeBuffer&& Other) noexcept;

	// Creates a new buffer
	HORDE_API bool CreateNew(const FParams& Params);

	// Opens an existing shared memory buffer (typically from handles created in another process)
	HORDE_API bool OpenExisting(const char* Name);

	// Close the current buffer and release all allocated resources
	HORDE_API void Close();

	// Test if the buffer is currently open
	bool IsValid() const { return Detail != nullptr; }

	// Creates a new reader for this buffer
	HORDE_API FComputeBufferReader CreateReader();

	// Creates a new writer for this buffer
	HORDE_API FComputeBufferWriter CreateWriter();

private:
	friend class FWorkerComputeSocket;

	FComputeBufferDetail* Detail;

	const char* GetName() const;
};

//
// Facilitates reading data from a compute buffer
//
class FComputeBufferReader
{
public:
	HORDE_API FComputeBufferReader();
	HORDE_API FComputeBufferReader(const FComputeBufferReader& Other);
	HORDE_API FComputeBufferReader(FComputeBufferReader&& Other) noexcept;
	HORDE_API ~FComputeBufferReader();

	HORDE_API FComputeBufferReader& operator=(const FComputeBufferReader& Other);
	HORDE_API FComputeBufferReader& operator=(FComputeBufferReader&& Other) noexcept;

	// Closes the handle to the underlying reader instance, resetting this instance back to empty
	HORDE_API void Close();

	// Detaches this reader from the buffer, causing all pending and subsequent reads to return immediately.
	HORDE_API void Detach();

	// Test if the reader is valid
	bool IsValid() const { return Detail != nullptr; }

	// Test whether the buffer has finished being written to (ie. MarkComplete() has been called by the writer) and all data has been read from it.
	HORDE_API bool IsComplete() const;

	// Move the read cursor forwards by the given number of bytes
	HORDE_API void AdvanceReadPosition(size_t Size);

	// Gets the amount of data that is ready to be read from a contiguous block of memory.
	HORDE_API size_t GetMaxReadSize() const;

	// Reads data into the given buffer
	HORDE_API size_t Read(void* Buffer, size_t MaxSize, int TimeoutMs = -1, bool* bOutTimedOut = nullptr);

	// Waits until the given amount of data has been read, and returns a pointer to it. Returns nullptr if the timeout expires, or
	// if the requested amount of data is not in a contiguous block of memory.
	HORDE_API const unsigned char* WaitToRead(size_t MinSize, int TimeoutMs = -1, bool* bOutTimedOut = nullptr);

private:
	struct FReaderRef;

	friend class FComputeBuffer;
	friend class FWorkerComputeSocket;

	FComputeBufferDetail* Detail;
	int ReaderIdx;

	FComputeBufferReader(FComputeBufferDetail* Detail, int ReaderIdx);
	const char* GetName() const;
};

//
// Facilitates writing data to a compute buffer
//
class FComputeBufferWriter
{
public:
	HORDE_API FComputeBufferWriter();
	HORDE_API FComputeBufferWriter(const FComputeBufferWriter& Other);
	HORDE_API FComputeBufferWriter(FComputeBufferWriter&& Other) noexcept;
	HORDE_API ~FComputeBufferWriter();

	HORDE_API FComputeBufferWriter& operator=(const FComputeBufferWriter& Other);
	HORDE_API FComputeBufferWriter& operator=(FComputeBufferWriter&& Other) noexcept;

	// Closes the handle to the underlying writer instance, resetting this instance back to empty
	HORDE_API void Close();

	// Test if the writer is valid
	bool IsValid() const { return Detail != nullptr; }

	// Signal that we've finished writing to this buffer
	HORDE_API void MarkComplete();

	// Move the write cursor forward by the given number of bytes
	HORDE_API void AdvanceWritePosition(size_t Size);

	// Gets the length of the current write buffer.
	HORDE_API size_t GetMaxWriteSize() const;

	// Get the max length of a chunk.
	HORDE_API size_t GetChunkMaxLength() const;

	// Writes data into the compute buffer
	HORDE_API size_t Write(const void* Buffer, size_t MaxSize, int TimeoutMs = -1);

	// Waits until a write buffer of the requested size is available, and returns a pointer to it. Returns nullptr if the 
	// timeout expires before enough data has been flushed to return a buffer of the given length.
	HORDE_API unsigned char* WaitToWrite(size_t MinSize, int TimeoutMs = -1);

private:
	friend class FComputeBuffer;
	friend class FWorkerComputeSocket;

	FComputeBufferDetail* Detail;

	FComputeBufferWriter(FComputeBufferDetail* Detail);
	const char* GetName() const;
};
