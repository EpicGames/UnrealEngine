// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ComputeBuffer.h"

class FComputeSocket;

//
// Allows bi-directional communication between two nodes using compute buffers
//
class FComputeChannel
{
public:
	// Reader for the channel
	FComputeBufferReader Reader;

	// Writer for the channel
	FComputeBufferWriter Writer;

	HORDE_API FComputeChannel();
	HORDE_API FComputeChannel(FComputeBufferReader InReader, FComputeBufferWriter InWriter);
	HORDE_API ~FComputeChannel();

	// Tests whether the channel is valid
	HORDE_API bool IsValid() const;

	// Sends bytes to the remote. 
	HORDE_API size_t Send(const void* Data, size_t Size, int TimeoutMs = -1);

	// Reads as many bytes as are available from the socket.
	HORDE_API size_t Recv(void* Data, size_t Size, int TimeoutMs = -1);

	// Indicate to the remote that no more data will be sent.
	HORDE_API void MarkComplete();
};
