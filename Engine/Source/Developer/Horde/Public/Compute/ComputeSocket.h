// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ComputeBuffer.h"
#include "ComputeChannel.h"
#include "ComputeTransport.h"
#include <vector>

//
// Connection to a remote machine that multiplexes data into and out-of multiple buffers
// attached to different channel numbers.
//
class FComputeSocket
{
public:
	HORDE_API FComputeSocket();
	HORDE_API virtual ~FComputeSocket();

	FComputeSocket(const FComputeSocket&) = delete;
	FComputeSocket& operator=(const FComputeSocket&) = delete;

	// Begins communication with the agent
	virtual void StartCommunication() = 0;

	// Attaches a new buffer for receiving data
	virtual void AttachRecvBuffer(int ChannelId, FComputeBuffer RecvBuffer) = 0;

	// Attaches a new buffer for sending data */
	virtual void AttachSendBuffer(int ChannelId, FComputeBuffer SendBuffer) = 0;

	// Attaches a channel to this socket
	HORDE_API TSharedPtr<FComputeChannel> CreateChannel(int ChannelId, bool Anonymous = false);

	// Attaches a channel to this socket
	HORDE_API TSharedPtr<FComputeChannel> CreateChannel(int ChannelId, FComputeBuffer RecvBuffer, FComputeBuffer SendBuffer);
};

//
// Socket used by a worker process to communicate with a host running on the same machine
// using shared memory to attach new buffers.
//
class FWorkerComputeSocket final : public FComputeSocket
{
public:
	static const char* const IpcEnvVar;

	HORDE_API FWorkerComputeSocket();
	HORDE_API ~FWorkerComputeSocket();

	// Opens a connection to the agent process using a command buffer read from an environment variable (EnvVarName)
	HORDE_API bool Open();

	// Opens a connection to the agent process using a specific command buffer name
	HORDE_API bool Open(const char* CommandBufferName);

	// Close the current connection
	HORDE_API void Close();

	// Begin communication with agent.
	HORDE_API virtual void StartCommunication() override;

	// Attaches a new buffer for receiving data
	HORDE_API virtual void AttachRecvBuffer(int ChannelId, FComputeBuffer RecvBuffer) override;

	// Attaches a new buffer for sending data
	HORDE_API virtual void AttachSendBuffer(int ChannelId, FComputeBuffer SendBuffer) override;

	// Reads and handles a command from the command buffer
	HORDE_API static void RunServer(FComputeBufferReader& CommandBufferReader, FComputeSocket& Socket);

private:
	enum class EMessageType;

	FComputeBufferWriter CommandBufferWriter;
	std::vector<FComputeBuffer> Buffers;

	void AttachBuffer(int ChannelId, EMessageType Type, const char* Name);

	static size_t ReadVarUInt(const unsigned char* Pos, unsigned int* OutValue);
	static size_t ReadString(const unsigned char* Pos, char* OutText, size_t OutTextMaxLen);

	static size_t WriteVarUInt(unsigned char* Pos, unsigned int Value);
	static size_t WriteString(unsigned char* Pos, const char* Text);
};

//
// Enum identifying which end of the socket a particular machine is
//
enum class EComputeSocketEndpoint
{
	// The initiating machine
	Local,

	// The remote machine
	Remote
};

// Creates a socket using a custom transport. Also returns the default channel (channel 0)
HORDE_API TUniquePtr<FComputeSocket> CreateComputeSocket(TUniquePtr<FComputeTransport> Transport, EComputeSocketEndpoint Endpoint);
