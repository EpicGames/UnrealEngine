// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ComputeBuffer.h"

//
// Interface for a transport mechanism that can be used by a compute socket
//
class FComputeTransport
{
public:
	HORDE_API virtual ~FComputeTransport();

	// Returns true if this compute transport channel has been successfully established.
	virtual bool IsValid() const = 0;
	
	// Sends data to the remote
	virtual size_t Send(const void* Data, size_t Size) = 0;

	// Receives data from the remote
	virtual size_t Recv(void* Data, size_t Size) = 0;

	// Indicates to the remote that no more data will be sent.
	virtual void MarkComplete() = 0;

	// Indicates that no more data will be sent or received, and that any blocking reads/writes should stop.
	virtual void Close() = 0;

	// Sends data to the remote, blocking until the entire message has been sent.
	HORDE_API bool SendMessage(const void* Data, size_t Size);

	// Receives a fixed length block of data from the remote, blocking until the entire length has been received.
	HORDE_API bool RecvMessage(void* Data, size_t Size);
};

//
// Implementation of FComputeTransport which uses in-memory buffers to transport data
//
class FBufferTransport final : public FComputeTransport
{
public:
	HORDE_API FBufferTransport(FComputeBufferWriter InSendBufferWriter, FComputeBufferReader InRecvBufferReader);

protected:
	virtual bool IsValid() const override final;
	virtual size_t Send(const void* Data, size_t Size) override final;
	virtual size_t Recv(void* Data, size_t Size) override final;
	virtual void MarkComplete() override final;
	virtual void Close() override final;

private:
	FComputeBufferWriter SendBufferWriter;
	FComputeBufferReader RecvBufferReader;
};
