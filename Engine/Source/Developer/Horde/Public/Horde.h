// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if !defined(HORDE_API)
#define HORDE_API
#endif

#include "Containers/UnrealString.h"

struct FHorde
{
	// Gets the server URL. Equivalent to calling IDesktopPlatform::GetHordeUrl().
	HORDE_API static bool GetServerUrl(FString& OutUrl, FString* OutUrlConfigSource = nullptr);

	// Settings available from the environment when running under a job
	HORDE_API static FString GetTemplateName();
	HORDE_API static FString GetTemplateId();
	HORDE_API static FString GetServerURL();
	HORDE_API static FString GetJobId();
	HORDE_API static FString GetJobURL();
	HORDE_API static FString GetStepId();
	HORDE_API static FString GetStepURL();
	HORDE_API static FString GetStepName();
	HORDE_API static FString GetBatchId();
};

