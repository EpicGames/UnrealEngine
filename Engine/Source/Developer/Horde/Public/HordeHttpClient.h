// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/UnrealString.h"
#include "Interfaces/IHttpRequest.h"
#include "Interfaces/IHttpResponse.h"

#define UE_API HORDE_API

class FHordeHttpClient
{
public:
	UE_API FHordeHttpClient(FString InServerUrl);
	UE_API ~FHordeHttpClient();

	UE_API bool Login(bool bUnattended, FFeedbackContext* Warn = nullptr);

	UE_API bool LoginWithOidc(const TCHAR* Profile, bool bUnattended, FFeedbackContext* Warn = nullptr);

	UE_API bool LoginWithEnvironmentVariable();
	
	UE_API TSharedRef<IHttpRequest> CreateRequest(const TCHAR* Verb, const TCHAR* Path);
	UE_API TSharedRef<IHttpResponse> Get(const TCHAR* Path);

	template<typename T>
	T Get(const TCHAR* Path)
	{
		T Result;
		Result.FromJson(Get(Path)->GetContentAsString());
		return Result;
	}

	static UE_API TSharedRef<IHttpResponse> ExecuteRequest(TSharedRef<IHttpRequest> Request);

private:
	FString ServerUrl;
	FString Token;
};

#undef UE_API
