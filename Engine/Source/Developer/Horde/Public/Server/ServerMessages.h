// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/JsonSerializerMacros.h"

/** Server Info */
struct FGetServerInfoResponse : FJsonSerializable
{
	/** Server version info */
	FString ServerVersion;

	/** The current agent version string */
	FString AgentVersion;

	/** The operating system server is hosted on */
	FString OsDescription;

	FGetServerInfoResponse();
	virtual ~FGetServerInfoResponse() override;
	virtual void Serialize(FJsonSerializerBase& Serializer, bool bFlatObject) override;
};

/** Describes the auth config for this server */
struct FGetAuthConfigResponse : FJsonSerializable
{
	/** Issuer for tokens from the auth provider */
	FString Method;

	/** Issuer for tokens from the auth provider */
	FString ServerUrl;

	/** Client id for the OIDC authority */
	FString ClientId;

	/** Optional redirect url provided to OIDC login for external tools (typically to a local server) */
	TArray<FString> LocalRedirectUrls;

	FGetAuthConfigResponse();
	virtual ~FGetAuthConfigResponse() override;
	virtual void Serialize(FJsonSerializerBase& Serializer, bool bFlatObject) override;
};

