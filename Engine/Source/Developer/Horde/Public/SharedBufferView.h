// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Memory/SharedBuffer.h"

#define UE_API HORDE_API

//
// View into a buffer with shared ownership
//
class FSharedBufferView final
{
public:
	UE_API FSharedBufferView();
	UE_API FSharedBufferView(FSharedBuffer InBuffer);
	UE_API FSharedBufferView(FSharedBuffer InBuffer, const FMemoryView& InView);
	UE_API FSharedBufferView(FSharedBuffer InBuffer, size_t InOffset, size_t InLength);
	UE_API ~FSharedBufferView();

	static UE_API FSharedBufferView Copy(const FMemoryView& View);

	UE_API FSharedBufferView Slice(uint64 Offset) const;
	UE_API FSharedBufferView Slice(uint64 Offset, uint64 Length) const;

	UE_API const unsigned char* GetPointer() const;
	UE_API size_t GetLength() const;
	UE_API FMemoryView GetView() const;

private:
	FSharedBuffer Buffer;
	FMemoryView View;
};

#undef UE_API
