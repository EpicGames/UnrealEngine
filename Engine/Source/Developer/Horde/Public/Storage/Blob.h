// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BlobHandle.h"
#include "BlobType.h"
#include "../SharedBufferView.h"

#define UE_API HORDE_API

/**
 * Describes a blob of data in the storage system
 */
struct FBlob
{
	/** Type of the blob. */
	FBlobType Type;

	/** Data for the blob. */
	FSharedBufferView Data;

	/** References to other blobs. */
	TArray<FBlobHandle> References;

	UE_API FBlob(const FBlobType& InType, FSharedBufferView InData, TArray<FBlobHandle> InReferences);
	UE_API ~FBlob();
};

#undef UE_API
