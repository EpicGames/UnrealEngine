// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SharedBufferView.h"
#include "BlobType.h"
#include "IO/IoHash.h"

#define UE_API HORDE_API

struct FBlob;
class FBlobHandleData;
class FBlobLocator;

/**
 * Handle to a blob. Can be used to reference blobs that have not been flushed yet. Implemented as a shared pointer to a FBlobHandleData object.
 */
class FBlobHandle : public TSharedPtr<FBlobHandleData>
{
public:
	using Super = TSharedPtr<FBlobHandleData>;
	using Super::operator bool;
	using Super::operator->;
	using Super::operator*;

	FBlobHandle()
	{ }

	FBlobHandle(TSharedPtr<FBlobHandleData>&& Other)
		: TSharedPtr<FBlobHandleData>(Other)
	{ }

	FBlobHandle(TSharedRef<FBlobHandleData> Value)
		: TSharedPtr<FBlobHandleData>(MoveTemp(Value))
	{ }

	template<typename T>
	FBlobHandle(TSharedPtr<T> Value)
		: TSharedPtr<FBlobHandleData>(StaticCastSharedPtr<FBlobHandleData, T>(MoveTemp(Value)))
	{ }

	template<typename T>
	FBlobHandle(TSharedRef<T> Value)
		: TSharedPtr<FBlobHandleData>(StaticCastSharedPtr<FBlobHandleData, T>(Value.ToSharedPtr()))
	{ }

	UE_API bool operator==(const FBlobHandle& Other) const;
	UE_API bool operator!=(const FBlobHandle& Other) const;

	friend uint32 GetTypeHash(const FBlobHandle& Handle);
};

/**
 * Handle to a blob with a particular state representation
 */
template<typename T>
class TBlobHandle : public FBlobHandle
{
public:
	TBlobHandle()
	{ }

	TBlobHandle(TSharedRef<T> Other)
		: FBlobHandle(MoveTemp(Other))
	{ }

	T* operator->() { return static_cast<T*>(FBlobHandle::operator->()); }
	const T* operator->() const { return static_cast<const T*>(FBlobHandle::operator->()); }
};

/**
 * Base interface for a blob.
 */
class FBlobHandleData
{
public:
	UE_API virtual ~FBlobHandleData();

	/** Gets the type name of this derived blob handle instance. */
	virtual const char* GetType() const = 0;

	/** Determines if this blob equals the other. */
	virtual bool Equals(const FBlobHandleData& Other) const = 0;

	/** Gets a hash for this blob. */
	virtual uint32 GetHashCode() const = 0;

	/** For a blob nested within another blob, gets a handle to the containing blob (eg. For a bundle node, will return the packet. For a bundle packet, will return the bundle. For a bundle or other non-nested blob, returns null.) */
	virtual FBlobHandle GetOuter() const = 0;

	/** Reads the blob's data. */
	virtual FBlob Read() const = 0;

	/** Gets the type of this blob. */
	UE_API virtual FBlobType ReadType() const;

	/** Gets the outward references from this blob. */
	UE_API virtual void ReadImports(TArray<FBlobHandle>& OutImports) const;

	/** Reads part of the blob, and returns a handle that can be used to access the data. */
	UE_API virtual FSharedBufferView ReadBody() const;

	/** Reads part of the blob, and returns a handle that can be used to access the data. */
	UE_API virtual FSharedBufferView ReadBody(size_t Offset, TOptional<size_t> Length = TOptional<size_t>()) const;

	/** Gets a locator for this blob, by calling TryAppendLocator recursively. */
	UE_API FBlobLocator GetLocator() const;

	/** Gets an identifier for this blob, relative to its outer. */
	virtual bool TryAppendIdentifier(FUtf8String& OutBuffer) const = 0;

	/** Gets a handle to a nested blob object. */
	UE_API virtual FBlobHandle GetFragmentHandle(const FUtf8StringView& Fragment) const;
};

/**
 * Stores a blob handle along with a hash of the target node.
 */
struct FBlobHandleWithHash
{
	FBlobHandle Handle;
	FIoHash Hash;

	FBlobHandleWithHash(FBlobHandle InHandle, FIoHash InHash)
		: Handle(MoveTemp(InHandle))
		, Hash(InHash)
	{ }
};

#undef UE_API
