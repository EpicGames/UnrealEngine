// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Utf8String.h"

#define UE_API HORDE_API

/**
 * Identifies the location of a blob. Meaning of this string is implementation defined.
 */
class FBlobLocator
{
public:
	UE_API FBlobLocator();
	UE_API FBlobLocator(FUtf8String InPath);
	UE_API FBlobLocator(const FBlobLocator& InBaseLocator, const FUtf8StringView& InFragment);
	UE_API explicit FBlobLocator(const FUtf8StringView& InPath);

	/** Tests whether this locator is valid. */
	UE_API bool IsValid() const;

	/** Gets the path for this locator. */
	UE_API const FUtf8String& GetPath() const;

	/** Gets the base locator for this blob. */
	UE_API FBlobLocator GetBaseLocator() const;

	/** Gets the path for this locator. */
	UE_API FUtf8StringView GetFragment() const;

	/** Determines if this locator can be unwrapped into an outer locator/fragment pair. */
	UE_API bool CanUnwrap() const;

	/** Split this locator into a locator and fragment. */
	UE_API bool TryUnwrap(FBlobLocator& OutLocator, FUtf8StringView& OutFragment) const;

	UE_API bool operator==(const FBlobLocator& Other) const;
	UE_API bool operator!=(const FBlobLocator& Other) const;

	friend uint32 GetTypeHash(const FBlobLocator& Locator);

private:
	FUtf8String Path;
};

#undef UE_API
