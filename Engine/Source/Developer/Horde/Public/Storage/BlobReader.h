// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Blob.h"
#include "BlobHandle.h"
#include "BlobLocator.h"
#include "RefName.h"

#define UE_API HORDE_API

struct FIoHash;

/**
 * Reads data from a blob object
 */
class FBlobReader
{
public:
	UE_API FBlobReader(const FBlob& InBlob);
	UE_API FBlobReader(int32 InVersion, const FMemoryView& InBuffer, const TArray<FBlobHandle>& InImports);

	/** Gets a version number for the current blob */
	UE_API int GetVersion() const;

	/** Gets a pointer to the remaining memory. */
	UE_API const unsigned char* GetBuffer() const;

	/** Gets the remaining memory to read from. */
	UE_API FMemoryView GetView() const;

	/** Advance the current read position. */
	UE_API void Advance(size_t Size);

	/** Reads the next import. */
	UE_API FBlobHandle ReadImport();

private:
	int32 Version;
	FMemoryView Buffer;
	const TArray<FBlobHandle>& Imports;
	int32 NextImportIdx;
};

// ------------------------------------------------------------------------

HORDE_API FBlobHandle ReadBlobHandle(FBlobReader& Reader);
HORDE_API FBlobHandleWithHash ReadBlobHandleWithHash(FBlobReader& Reader);
HORDE_API int ReadInt32(FBlobReader& Reader);
HORDE_API FIoHash ReadIoHash(FBlobReader& Reader);
HORDE_API FMemoryView ReadFixedLengthBytes(FBlobReader& Reader, size_t Length);
HORDE_API size_t ReadUnsignedVarInt(FBlobReader& Reader);
HORDE_API FUtf8String ReadString(FBlobReader& Reader);
HORDE_API FMemoryView ReadStringSpan(FBlobReader& Reader);

#undef UE_API
