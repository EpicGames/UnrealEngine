// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API HORDE_API

/**
 * Identifies the type of a blob
 */
struct FBlobType
{
	static UE_API const FBlobType Leaf;

	/** Nominal identifier for the type. */
	FGuid Guid;

	/** Version number for the serializer. */
	int Version;

	UE_API FBlobType(const FGuid& InGuid, int InVersion);

	UE_API bool operator==(const FBlobType& Other) const;
	UE_API bool operator!=(const FBlobType& Other) const;
};

#undef UE_API
