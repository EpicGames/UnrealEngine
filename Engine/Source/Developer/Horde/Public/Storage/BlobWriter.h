// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Blob.h"
#include "BlobHandle.h"
#include "BlobLocator.h"
#include "RefName.h"
#include "../SharedBufferView.h"

#define UE_API HORDE_API

struct FAliasInfo;
struct FRefOptions;

/**
 * Interface for writing data to the storage system.
 */
class FBlobWriter
{
public:
	UE_API virtual ~FBlobWriter();

	/** Create another writer instance, allowing multiple threads to write in parallel. */
	virtual TUniquePtr<FBlobWriter> Fork() = 0;

	/** Adds an alias for the current blob. */
	virtual void AddAlias(const FAliasInfo& AliasInfo) = 0;

	/** Adds a reference to another blob. */
	virtual void AddImport(FBlobHandle Target) = 0;

	/** Adds a named reference to the blob being built. */
	UE_API void AddRef(const FRefName& RefName);
	virtual void AddRef(const FRefName& RefName, const FRefOptions& Options) = 0;

	/** Gets a block of memory of the given size. */
	UE_API void* GetOutputBuffer(size_t Size);

	/** Gets a block of memory, at least the given size. */
	virtual FMutableMemoryView GetOutputBufferAsSpan(size_t UsedSize, size_t DesiredSize) = 0;

	/** Advance the current write position. */
	virtual void Advance(size_t Size) = 0;

	/** Finish writing a blob that has been written into the output buffer. */
	virtual FBlobHandle CompleteBlob(const FBlobType& InType) = 0;

	/** Flush any pending nodes to storage. */
	virtual void Flush() = 0;
};

// ------------------------------------------------------------------------------------------

HORDE_API void WriteBlobHandle(FBlobWriter& Writer, FBlobHandle Handle);
HORDE_API void WriteBlobHandleWithHash(FBlobWriter& Writer, FBlobHandleWithHash Target);

HORDE_API void WriteIoHash(FBlobWriter& Writer, const struct FIoHash& Hash);

HORDE_API void WriteFixedLengthBytes(FBlobWriter& Writer, const void* Data, size_t Length);
HORDE_API void WriteFixedLengthBytes(FBlobWriter& Writer, const FMemoryView& View);

HORDE_API size_t MeasureUnsignedVarInt(size_t Value);
HORDE_API size_t WriteUnsignedVarInt(void* Buffer, size_t Value);
HORDE_API void WriteUnsignedVarIntWithKnownLength(void* Buffer, size_t Value, size_t NumBytes);
HORDE_API void WriteUnsignedVarInt(FBlobWriter& Writer, size_t Value);

HORDE_API size_t MeasureString(const char* Text);
HORDE_API size_t MeasureString(const FUtf8StringView& Text);
HORDE_API void WriteString(FBlobWriter& Writer, const char* Text);
HORDE_API void WriteString(FBlobWriter& Writer, const FUtf8StringView& Text);
HORDE_API void WriteString(FBlobWriter& Writer, const FUtf8String& Text);

#undef UE_API
