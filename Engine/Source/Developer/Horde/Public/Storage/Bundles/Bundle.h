// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Storage/Blob.h"

#define UE_API HORDE_API

/** Bundle version number */
enum class EBundleVersion : unsigned char
{
	/** Initial version number. */
	Initial = 0,

	/** Added the BundleExport.Alias property. */
	ExportAliases = 1,

	/** Back out change to include aliases. Will likely do this through an API rather than baked into the data. */
	RemoveAliases = 2,

	/** Use data structures which support in-place reading and writing. */
	InPlace = 3,

	/** Add import hashes to imported nodes. */
	ImportHashes = 4,

	/** Structure bundles as a sequence of self-contained packets (uses V2 code). */
	PacketSequence = 5,

	/** Last item in the enum. Used for Latest. */
	LatestPlusOne,

	/** The current version number. */
	Latest = (int)LatestPlusOne - 1,

	/** Last version using the V1 pipeline. */
	LatestV1 = ImportHashes,

	/** Last version using the V2 pipeline. */
	LatestV2 = Latest,
};

/**
 * General bundle properties
 */
struct FBundle
{
	static UE_API const FBlobType BlobType;
};

/**
 * Signature for a bundle
 */
struct FBundleSignature
{
	/** Number of bytes in a serialized signature. */
	static const int NumBytes = 8;

	/** Version number for the following file data. */
	EBundleVersion Version;

	/** Length of the packet, including the length of the signature itself. */
	size_t Length;

	UE_API FBundleSignature(EBundleVersion InVersion, size_t InLength);
	static UE_API FBundleSignature Read(const void* Data);
	UE_API void Write(void* Data);
};

#undef UE_API
