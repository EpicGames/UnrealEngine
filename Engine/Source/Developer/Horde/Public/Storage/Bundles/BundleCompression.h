// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Memory/MemoryView.h"

#define UE_API HORDE_API

/** Indicates the compression format in a bundle */
enum EBundleCompressionFormat : unsigned char
{
	/** Packets are uncompressed. */
	None = 0,

	/** LZ4 compression. */
	LZ4 = 1,

	/** Gzip compression. */
	Gzip = 2,

	/** Oodle compression (Kraken). */
	Oodle = 3,

	/** Brotli compression. */
	Brotli = 4,
};

/*
 * Utility methods for compressing bundles
 */
struct FBundleCompression
{
	/** Gets the maximum size of the buffer required to compress the given data. */
	static UE_API size_t GetMaxSize(EBundleCompressionFormat Format, const FMemoryView& Input);

	/** Compress a data packet. */
	static UE_API size_t Compress(EBundleCompressionFormat Format, const FMemoryView& Input, const FMutableMemoryView& Output);

	/** Decompress a packet of data. */
	static UE_API void Decompress(EBundleCompressionFormat Format, const FMemoryView& Input, const FMutableMemoryView& Output);
};

#undef UE_API
