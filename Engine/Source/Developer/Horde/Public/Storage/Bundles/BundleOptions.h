// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Bundle.h"
#include "BundleCompression.h"

#define UE_API HORDE_API

/*
 * Options for configuring a bundle serializer
 */
struct FBundleOptions
{
	/** Default options value. */
	static UE_API const FBundleOptions Default;

	/** Maximum version number of bundles to write. */
	EBundleVersion MaxVersion = EBundleVersion::LatestV2;

	/** Maximum payload size fo a blob. */
	int MaxBlobSize = 10 * 1024 * 1024;

	/** Compression format to use. */
	EBundleCompressionFormat CompressionFormat = EBundleCompressionFormat::LZ4;

	/** Minimum size of a block to be compressed. */
	int MinCompressionPacketSize = 16 * 1024;

	UE_API FBundleOptions();
};

#undef UE_API
