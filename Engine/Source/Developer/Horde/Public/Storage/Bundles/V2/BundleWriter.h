// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Utf8String.h"
#include "Storage/StorageClient.h"
#include "Storage/BlobWriter.h"
#include "Storage/ChunkedBufferWriter.h"
#include "Storage/Clients/BundleStorageClient.h"
#include "Storage/Clients/KeyValueStorageClient.h"

#define UE_API HORDE_API

class FPacketWriter;

/**
 * Writes blobs into bundles
 */
class FBundleWriter final : public FBlobWriter
{
public:
	UE_API FBundleWriter(TSharedRef<FKeyValueStorageClient> InStorageClient, FUtf8String InBasePath, const FBundleOptions& InOptions);
	UE_API ~FBundleWriter();

	// Inherited from FBlobWriter
	UE_API void Flush() override;
	UE_API TUniquePtr<FBlobWriter> Fork() override;
	UE_API virtual void AddAlias(const FAliasInfo& AliasInfo) override;
	UE_API virtual void AddImport(FBlobHandle Target) override;
	UE_API virtual void AddRef(const FRefName& RefName, const FRefOptions& Options) override;
	UE_API virtual FBlobHandle CompleteBlob(const FBlobType& InType) override;
	UE_API virtual FMutableMemoryView GetOutputBufferAsSpan(size_t UsedSize, size_t DesiredSize) override;
	UE_API virtual void Advance(size_t Size) override;

private:
	class FPendingExportHandleData;
	class FPendingPacketHandleData;
	class FPendingBundleHandleData;

	TSharedRef<FKeyValueStorageClient> StorageClient;
	FUtf8String BasePath;
	FBundleOptions Options;

	TBlobHandle<FPendingPacketHandleData> CurrentPacketHandle;
	TUniquePtr<FPacketWriter> PacketWriter;

	TBlobHandle<FPendingBundleHandleData> CurrentBundleHandle;
	FChunkedBufferWriter CompressedPacketWriter;
	TArray<FBlobHandle> CurrentBundleImports;

	TArray<FBlobHandle> BundleReferences;
	//	std::vector<std::pair<FBlobHandle, FAliasInfo>> PendingExportAliases;
	//	FChunkedBlobWriter EncodedPacketWriter;

	UE_API void StartPacket();
	UE_API void FinishPacket();
};

#undef UE_API
