// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Storage/BlobHandle.h"
#include "PacketHandle.h"

#define UE_API HORDE_API

class FExportHandleData;

/** Handle to an export within a packet. */
typedef TBlobHandle<FExportHandleData> FExportHandle;

/** 
 * Implementation of export handle data.
 */
class FExportHandleData final : public FBlobHandleData, public TSharedFromThis<FExportHandleData, ESPMode::ThreadSafe>
{
public:
	static UE_API const char Type[];

	UE_API FExportHandleData(FPacketHandle InPacket, int32 InExportIdx);
	UE_API FExportHandleData(FPacketHandle InPacket, const FUtf8StringView& InFragment);
	UE_API ~FExportHandleData();

	static UE_API void AppendIdentifier(FUtf8String& OutBuffer, int32 ExportIdx);

	// Implementation of FBlobHandle
	UE_API virtual bool Equals(const FBlobHandleData& Other) const override;
	UE_API virtual uint32 GetHashCode() const override;
	UE_API virtual FBlobHandle GetOuter() const override;
	UE_API virtual const char* GetType() const override;
	UE_API virtual FBlob Read() const override;
	UE_API virtual bool TryAppendIdentifier(FUtf8String& OutBuffer) const override;

private:
	static UE_API const FUtf8StringView FragmentPrefix;

	FPacketHandle Packet;
	int32 ExportIdx;

	static UE_API bool TryParse(const FUtf8StringView& Fragment, int32& OutExportIdx);
};

#undef UE_API
