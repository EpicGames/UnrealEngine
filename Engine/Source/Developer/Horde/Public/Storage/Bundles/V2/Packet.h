// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Utf8String.h"
#include "SharedBufferView.h"
#include "Storage/BlobType.h"
#include "Storage/Bundles/BundleCompression.h"

#define UE_API HORDE_API

class FChunkedBufferWriter;
struct FPacketImport;
struct FPacketExport;

/**
 * Accessor for data structures stored into a serialized bundle packet.
 */
class FPacket
{
public:
	static UE_API const FBlobType BlobType;

	UE_API FPacket(FSharedBufferView InBuffer);
	UE_API virtual ~FPacket();

	/** Gets the underlying buffer for this packet. */
	UE_API FSharedBufferView GetBuffer() const;

	/** Length of this packet. */
	UE_API size_t GetLength() const;

	/** Gets the number of types in this packet. */
	UE_API size_t GetTypeCount() const;

	/** Gets a type from the packet. */
	UE_API const FBlobType& GetType(size_t TypeIdx) const;

	/** Gets the number of imports in this packet. */
	UE_API size_t GetImportCount() const;

	/** Gets the locator for a particular import. */
	UE_API FPacketImport GetImport(size_t ImportIdx) const;

	/** Gets the number of exports in this packet. */
	UE_API size_t GetExportCount() const;

	/** Gets the bulk data for a particular export. */
	UE_API FPacketExport GetExport(size_t ExportIdx) const;

	/** Encodes a packet. */
	UE_API void Encode(EBundleCompressionFormat Format, FChunkedBufferWriter& Writer);

	/** Decodes a packet from the given data. */
	static UE_API FPacket Decode(const FMemoryView& View);

private:
	struct FEncodedPacketHeader;

	FSharedBufferView Buffer;
};

/**
 * Specifies the path to an imported node
 */
struct FPacketImport
{
	static constexpr int32 Bias = 3;
	static constexpr int32 InvalidBaseIdx = -1;
	static constexpr int32 CurrentPacketBaseIdx = -2;
	static constexpr int32 CurrentBundleBaseIdx = -3;

	UE_API FPacketImport(const FMemoryView& InView);
	UE_API FPacketImport(int32 InBaseIdx, const FUtf8StringView& InFragment);

	/** Gets the index of the base packet import. */
	UE_API int GetBaseIdx() const;

	/** Gets the fragment for this import. */
	UE_API FUtf8StringView GetFragment() const;

private:
	int32 BaseIdx;
	FUtf8StringView Fragment;
};

/*
 * Data for an exported node in a packet
 */
struct FPacketExport
{
	UE_API FPacketExport(FSharedBufferView InBuffer);
	UE_API ~FPacketExport();

	/** Gets the index of this export's type. */
	UE_API size_t GetTypeIndex() const;

	/** Gets the import indexes for this export. */
	UE_API void GetImportIndices(TArray<size_t>& OutIndices) const;

	/** Gets the payload for this export. */
	UE_API FSharedBufferView GetPayload() const;

private:
	FSharedBufferView Buffer;
	int32 TypeIdx;
	FMemoryView Imports;
};

#undef UE_API
