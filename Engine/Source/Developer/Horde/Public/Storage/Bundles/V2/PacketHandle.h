// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "Storage/Blob.h"
#include "Storage/BlobHandle.h"
#include "Storage/BlobType.h"
#include "Storage/StorageClient.h"

#define UE_API HORDE_API

class FPacketReader;
class FPacketHandleData;

// Handle to an packet within a bundle. 
typedef TBlobHandle<FPacketHandleData> FPacketHandle;

// Data for FPacketHandle
class FPacketHandleData final : public FBlobHandleData, public TSharedFromThis<FPacketHandleData, ESPMode::ThreadSafe>
{
public:
	static UE_API const char Type[];

	UE_API FPacketHandleData(TSharedRef<FStorageClient> InStorageClient, FBlobHandle InBundle, size_t InPacketOffset, size_t InPacketLength);
	UE_API FPacketHandleData(TSharedRef<FStorageClient> InStorageClient, FBlobHandle InBundle, const FUtf8StringView& InFragment);
	UE_API ~FPacketHandleData();

	/** Reads an export from this packet. */
	UE_API FBlob ReadExport(size_t ExportIdx) const;

	/** Reads an export body from this packet. */
	UE_API FSharedBufferView ReadExportBody(size_t ExportIdx) const;

	// Implementation of FBlobHandle
	UE_API virtual bool Equals(const FBlobHandleData& Other) const override;
	UE_API virtual uint32 GetHashCode() const override;
	UE_API virtual FBlobHandle GetOuter() const override;
	UE_API virtual const char* GetType() const override;
	UE_API virtual FBlob Read() const override;
	UE_API virtual bool TryAppendIdentifier(FUtf8String& OutBuffer) const override;
	UE_API virtual FBlobHandle GetFragmentHandle(const FUtf8StringView& Fragment) const override;

private:
	static UE_API const char FragmentPrefix[];
	static UE_API const size_t FragmentPrefixLength;

	TSharedRef<FStorageClient> StorageClient;

	FBlobHandle Bundle;
	size_t PacketOffset;
	size_t PacketLength;

	mutable TSharedPtr<FPacketReader> PacketReader;

	UE_API const FPacketReader& GetPacketReader() const;
	static UE_API bool TryParse(const FUtf8StringView& Fragment, size_t& OutPacketOffset, size_t& OutPacketLength);
};

#undef UE_API
