// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Storage/StorageClient.h"
#include "Packet.h"
#include "PacketHandle.h"

#define UE_API HORDE_API

/**
 * Utility class for constructing BlobData objects from a packet, caching any computed handles to other blobs.
 */
class FPacketReader
{
public:
	UE_API FPacketReader(TSharedRef<FStorageClient> InStorageClient, FBlobHandle InBundleHandle, FPacketHandle InPacketHandle, FPacket InPacket);
	UE_API ~FPacketReader();

	/** Read a blob object for the entire packet. */
	UE_API FBlob ReadPacket() const;

	/** Reads an export from this packet. */
	UE_API FBlob ReadExport(size_t ExportIdx) const;

	/** Reads an export body from this packet. */
	UE_API FSharedBufferView ReadExportBody(size_t ExportIdx) const;

private:
	TSharedRef<FStorageClient> StorageClient;
	FBlobHandle BundleHandle;
	FPacketHandle PacketHandle;
	FPacket Packet;
	mutable TArray<FBlobHandle> CachedImportHandles;

	UE_API FBlobHandle GetImportHandle(size_t Index) const;
};

#undef UE_API
