// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Storage/BlobHandle.h"
#include "Packet.h"
#include "PacketHandle.h"
#include "Storage/ChunkedBufferWriter.h"

#define UE_API HORDE_API

/**
 * Writes exports into a new bundle packet
 */
class FPacketWriter
{
public:
	UE_API FPacketWriter(FBlobHandle InBundleHandle, FBlobHandle InPacketHandle);
	FPacketWriter(const FPacketWriter&) = delete;
	UE_API ~FPacketWriter();

	/** Current length of the packet */
	UE_API size_t GetLength() const;

	/** Adds an import to the current blob */
	UE_API void AddImport(FBlobHandle Import);

	/** Gets the number of unique imports current added to this packet */
	UE_API int GetImportCount() const;

	/** Gets a packet import by index */
	UE_API FBlobHandle GetImport(int ImportIdx) const;

	/** Gets data to write new export */
	UE_API FMutableMemoryView GetOutputBuffer(size_t UsedSize, size_t DesiredSize);

	/** Increase the length of the current blob */
	UE_API void Advance(size_t Size);

	/** Writes a new blob to this packet */
	UE_API int CompleteBlob(const FBlobType& Type);

	/** Gets the number of exports currently in this packet */
	UE_API int GetExportCount() const;

	/** Reads data for a blob written to storage */
	UE_API FBlob GetExport(int ExportIdx) const;

	/** Mark the current packet as complete */
	UE_API FPacket CompletePacket();

	FPacketWriter& operator=(const FPacketWriter&) = delete;

private:
	FBlobHandle BundleHandle;
	FBlobHandle PacketHandle;

	FSharedBuffer Buffer;
	uint8* BufferBasePtr;
	size_t Length;

	size_t NextBlobLength;
	TArray<int32> NextBlobImports;

	TArray<FBlobType> Types;
	TArray<uint32> ImportOffsets;
	FChunkedBufferWriter ImportWriter;
	TArray<FBlobHandle> ImportHandles;
	TMap<FBlobHandle, int> ImportMap;
	TArray<uint32> ExportOffsets;

	UE_API FMutableMemoryView GetOutputBufferInternal(size_t UsedSize, size_t DesiredSize);
	UE_API uint8* GetOutputSpanAndAdvance(size_t Length);

	UE_API int32 FindOrAddType(FBlobType Type);
	UE_API int32 FindOrAddImport(const FBlobHandle& Handle);

	UE_API void WriteTypeTable();
	UE_API void WriteImportTable();
	UE_API void WriteExportTable();

	static UE_API size_t Align(size_t Offset);
};

#undef UE_API
