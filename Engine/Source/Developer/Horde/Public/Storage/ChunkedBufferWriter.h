// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BlobWriter.h"

#define UE_API HORDE_API

/**
 * Writes data to a series of memory blocks
 */
class FChunkedBufferWriter
{
public:
	UE_API FChunkedBufferWriter(size_t InitialSize = 1024);
	FChunkedBufferWriter(const FChunkedBufferWriter&) = delete;
	FChunkedBufferWriter& operator = (const FChunkedBufferWriter&) = delete;
	UE_API virtual ~FChunkedBufferWriter();

	/** Reset the contents of this writer. */
	UE_API void Reset();

	/** Gets the current written length of this buffer. */
	UE_API size_t GetLength() const;

	/** Get a handle to part of the written buffer */
	UE_API FSharedBufferView Slice(size_t Offset, size_t Length) const;

	/** Gets a view over the underlying buffer. */
	UE_API TArray<FMemoryView> GetView() const;

	/** Copies the entire contents of this writer to another buffer. */
	UE_API void CopyTo(void* Buffer) const;

	/** Gets an output buffer for writing. */
	UE_API FMutableMemoryView GetOutputBuffer(size_t UsedSize, size_t DesiredSize);

	/** Increase the length of the written data. */
	UE_API void Advance(size_t Size);

private:
	struct FChunk;

	TArray<FChunk> Chunks;
	size_t WrittenLength;
};

#undef UE_API
