// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Storage/StorageClient.h"
#include "Storage/BlobHandle.h"
#include "Storage/Bundles/BundleOptions.h"
#include "Storage/Clients/KeyValueStorageClient.h"

#define UE_API HORDE_API

/**
 * Base class for an implementation of <see cref="IStorageClient"/>, providing implementations for some common functionality using bundles.
 */
class FBundleStorageClient final : public FKeyValueStorageClient
{
public:
	UE_API FBundleStorageClient(TSharedRef<FKeyValueStorageClient> InInner);
	UE_API virtual ~FBundleStorageClient() override;

	// FKeyValueStorageClient implementation
	UE_API virtual FBlob ReadBlob(const FBlobLocator& Locator) const override;
	UE_API virtual FBlobHandle WriteBlob(const FUtf8StringView& BasePath, const FBlobType& Type, const TArrayView<FMemoryView>& Data, const TArrayView<FBlobHandle>& Imports) override;

	// FBundleStorageClient implementation
	UE_API virtual FBlobHandle CreateHandle(const FBlobLocator& Locator) const override;
	UE_API virtual TUniquePtr<FBlobWriter> CreateWriter(FUtf8String BasePath) override;
	UE_API TUniquePtr<FBlobWriter> CreateWriter(FUtf8String BasePath, const FBundleOptions& Options);

	UE_API virtual void AddAlias(const char* Name, FBlobAlias Alias) override;
	UE_API virtual void RemoveAlias(const char* Name, FBlobHandle Handle) override;
	UE_API virtual void FindAliases(const char* Name, int MaxResults, TArray<FBlobAlias>& OutAliases) override;

	UE_API virtual bool DeleteRef(const FRefName& Name) override;
	UE_API virtual FBlobHandle ReadRef(const FRefName& Name, const FRefCacheTime& CacheTime) const override;
	UE_API virtual void WriteRef(const FRefName& Name, const FBlobHandle& Handle, const FRefOptions& Options) override;

private:
	class FBundleHandleData;
	friend FBundleHandleData;

	TSharedRef<FKeyValueStorageClient> Inner;

	UE_API TBlobHandle<FBundleHandleData> CreateBundleHandle(const FBlobLocator& Locator) const;
	UE_API FBlobHandle GetFragmentHandle(const FBlobHandle& BundleHandle, const FUtf8StringView& Fragment) const;
};

#undef UE_API
