// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "KeyValueStorageClient.h"
#include <filesystem>

#define UE_API HORDE_API

/**
 * Implementation of FStorageClient which writes data to files on disk.
 */
class FFileStorageClient : public FKeyValueStorageClient
{
public:
	UE_API FFileStorageClient(std::filesystem::path InRootDir);
	UE_API ~FFileStorageClient();

	static UE_API FBlobLocator ReadRefFromFile(const std::filesystem::path &File);
	static UE_API void WriteRefToFile(const std::filesystem::path& File, const FBlobLocator& Locator);

	UE_API std::filesystem::path GetBlobFile(const FBlobLocator& Locator) const;
	UE_API std::filesystem::path GetRefFile(const FRefName& Name) const;

	UE_API virtual void AddAlias(const char* Name, FBlobAlias Alias) override;
	UE_API virtual void RemoveAlias(const char* Name, FBlobHandle Handle) override;
	UE_API virtual void FindAliases(const char* Name, int MaxResults, TArray<FBlobAlias>& OutAliases) override;

	UE_API virtual bool DeleteRef(const FRefName& Name) override;
	UE_API virtual FBlobHandle ReadRef(const FRefName& Name, const FRefCacheTime& CacheTime) const override;
	UE_API virtual void WriteRef(const FRefName& Name, const FBlobHandle& Handle, const FRefOptions& Options) override;

protected:
	UE_API virtual FBlob ReadBlob(const FBlobLocator& Locator) const override;
	UE_API virtual FBlobHandle WriteBlob(const FUtf8StringView& BasePath, const FBlobType& Type, const TArrayView<FMemoryView>& Data, const TArrayView<FBlobHandle>& Imports) override;

private:
	std::filesystem::path RootDir;
};

#undef UE_API
