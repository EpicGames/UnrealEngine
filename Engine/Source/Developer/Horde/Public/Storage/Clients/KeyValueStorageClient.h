// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Storage/StorageClient.h"
#include "Storage/BlobHandle.h"

#define UE_API HORDE_API

/*
 * Base class for storage clients that wrap a diirect key/value type store without any merging/splitting.
 */
class FKeyValueStorageClient : public FStorageClient
{
public:
	/** Read a single blob from the underlying store. */
	virtual FBlob ReadBlob(const FBlobLocator& Locator) const = 0;

	/** Write a single blob to the underlying store. */
	virtual FBlobHandle WriteBlob(const FUtf8StringView& BasePath, const FBlobType& Type, const TArrayView<FMemoryView>& Data, const TArrayView<FBlobHandle>& Imports) = 0;

	// Implementation of FStorageClient
	UE_API virtual FBlobHandle CreateHandle(const FBlobLocator& Locator) const override;
	UE_API virtual TUniquePtr<FBlobWriter> CreateWriter(FUtf8String BasePath = FUtf8String()) override;

private:
	class FHandleData;
};

#undef UE_API
