// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "KeyValueStorageClient.h"

#define UE_API HORDE_API

/**
 * Implementation of FStorageClient which stores data in memory.
 */
class FMemoryStorageClient final : public FKeyValueStorageClient
{
public:
	UE_API FMemoryStorageClient();
	UE_API ~FMemoryStorageClient();

	UE_API virtual void AddAlias(const char* Name, FBlobAlias Alias) override;
	UE_API virtual void RemoveAlias(const char* Name, FBlobHandle Handle) override;
	UE_API virtual void FindAliases(const char* Name, int MaxResults, TArray<FBlobAlias>& OutAliases) override;

	UE_API virtual bool DeleteRef(const FRefName& Name) override;
	UE_API virtual FBlobHandle ReadRef(const FRefName& Name, const FRefCacheTime& CacheTime) const override;
	UE_API virtual void WriteRef(const FRefName& Name, const FBlobHandle& Target, const FRefOptions& Options) override;

protected:
	UE_API virtual FBlob ReadBlob(const FBlobLocator& Locator) const override;
	UE_API virtual FBlobHandle WriteBlob(const FUtf8StringView& BasePath, const FBlobType& Type, const TArrayView<FMemoryView>& Data, const TArrayView<FBlobHandle>& Imports) override;

private:
	mutable FCriticalSection CriticalSection;

	TMap<FBlobLocator, FBlob> Blobs;
	TMap<FRefName, FBlobLocator> Refs;
	TMap<FUtf8String, TArray<FBlobAlias>> Aliases;
};

#undef UE_API
