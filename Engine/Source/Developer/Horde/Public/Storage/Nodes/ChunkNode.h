// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IO/IoHash.h"
#include "Storage/BlobHandle.h"
#include "Storage/BlobType.h"
#include "Storage/BlobWriter.h"
#include "SharedBufferView.h"
#include "Hash/Blake3.h"
#include "Hash/BuzHash.h"

#define UE_API HORDE_API

/**
 * Options for chunking data
 */

struct FChunkingOptions
{
	static UE_API const FChunkingOptions Default;

	int32 MinChunkSize = 4 * 1024;
	int32 TargetChunkSize = 64 * 1024;
	int32 MaxChunkSize = 128 * 1024;
};

/**
 * Node containing a chunk of data
 */
class FChunkNode
{
public:
	static UE_API const FBlobType LeafBlobType;
	static UE_API const FBlobType InteriorBlobType;

	TArray<FBlobHandleWithHash> Children;
	FSharedBufferView Data;

	UE_API FChunkNode();
	UE_API FChunkNode(TArray<FBlobHandleWithHash> InChildren, FSharedBufferView InData);
	UE_API ~FChunkNode();

	static UE_API FChunkNode Read(FBlob Blob);
	UE_API FBlobHandleWithHash Write(FBlobWriter& Writer) const;

	static UE_API FBlobHandleWithHash Write(FBlobWriter& Writer, const TArrayView<const FBlobHandleWithHash>& Children, FMemoryView Data);
};

/**
 * Utility class for reading data a data stream from a tree of chunk nodes
 */
class FChunkNodeReader
{
public:
	UE_API FChunkNodeReader(FBlob Blob);
	UE_API FChunkNodeReader(const FBlobHandle& Handle);
	UE_API ~FChunkNodeReader();

	UE_API bool IsComplete() const;
	UE_API FMemoryView GetBuffer() const;
	UE_API void Advance(int32 Length);

	UE_API operator bool() const;

private:
	struct FStackEntry
	{
		FBlob Blob;
		size_t Position;

		FStackEntry(FBlob InBlob)
			: Blob(MoveTemp(InBlob))
			, Position(0)
		{ }
	};
	TArray<FStackEntry> Stack;
};

/**
 * Utility class for writing new data to a tree of chunk nodes
 */
class FChunkNodeWriter
{
public:
	UE_API FChunkNodeWriter(FBlobWriter& InWriter, const FChunkingOptions& InOptions = FChunkingOptions::Default);
	UE_API ~FChunkNodeWriter();

	UE_API void Write(FMemoryView Data);

	UE_API FBlobHandleWithHash Flush(FIoHash& OutStreamHash);

private:
	FBlobWriter& Writer;
	const FChunkingOptions& Options;

	FBuzHash RollingHash;
	uint32 Threshold;
	int32 NodeLength;

	FBlake3 StreamHasher;

	TArray<FBlobHandleWithHash> Nodes;

	UE_API void WriteNode();
};

#undef UE_API
