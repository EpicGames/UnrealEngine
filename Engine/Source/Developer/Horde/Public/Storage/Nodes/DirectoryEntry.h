// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Utf8String.h"
#include "../BlobHandle.h"
#include "IO/IoHash.h"

#define UE_API HORDE_API

class FBlobReader;
class FBlobWriter;

/**
 * Entry for a directory within a directory node
 */
class FDirectoryEntry
{
public:
	FBlobHandle Target;
	FIoHash TargetHash;

	/** Total size of this directory's contents. */
	int64 Length;

	/** Name of this directory. */
	const FUtf8String Name;

	UE_API FDirectoryEntry(FBlobHandle InTarget, const FIoHash& InTargetHash, FUtf8String InName, int64 InLength);
	UE_API ~FDirectoryEntry();

	static UE_API FDirectoryEntry Read(FBlobReader& Reader);
	UE_API void Write(FBlobWriter& Writer) const;
};

#undef UE_API
