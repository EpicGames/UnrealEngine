// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "FileEntry.h"
#include "DirectoryEntry.h"
#include "../BlobType.h"
#include "../StorageClient.h"

#define UE_API HORDE_API

/**
 * Flags for a directory node
 */
enum class EDirectoryFlags
{
	/** No flags specified. */
	None = 0,
};

/**
 * Stores the contents of a directory in a blob
 */
class FDirectoryNode
{
public:
	static UE_API const FBlobType BlobType;

	EDirectoryFlags Flags;
	TMap<FUtf8String, FFileEntry> NameToFile;
	TMap<FUtf8String, FDirectoryEntry> NameToDirectory;

	UE_API FDirectoryNode(EDirectoryFlags InFlags = EDirectoryFlags::None);
	UE_API ~FDirectoryNode();

	static UE_API FDirectoryNode Read(const FBlob& Blob);
	UE_API FBlobHandle Write(FBlobWriter& Writer) const;
};

#undef UE_API
