// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Utf8String.h"
#include "IO/IoHash.h"
#include "SharedBufferView.h"
#include "Storage/BlobHandle.h"

#define UE_API HORDE_API


class FBlobReader;
class FBlobWriter;

/**
 * Flags for a file entry
 */
enum class EFileEntryFlags
{
	/** No other flags set. */
	None = 0,

	/** Indicates that the referenced file is executable. */
	Executable = 4,

	/** File should be stored as read-only. */
	ReadOnly = 8,

	/** File contents are utf-8 encoded text. Client may want to replace line-endings with OS-specific format. */
	Text = 16,

	/** Used to indicate that custom data is included in the output. Used internally for serialization; not exposed to users. */
	HasCustomData = 32,

	/** File should be materialized as UTF-16 (but is stored as a UTF-8 source). */
	Utf16 = 64,
};

ENUM_CLASS_FLAGS(EFileEntryFlags)

/**
 * Entry for a file within a directory node
 */
class FFileEntry
{
public:
	/** Handle to the root chunked data node. */
	FBlobHandleWithHash Target;

	/** Name of this file. */
	const FUtf8String Name;

	/** Flags for this file. */
	EFileEntryFlags Flags;

	/** Length of this file. */
	int64 Length;

	/** Hash of the file. */
	FIoHash Hash;

	/** Custom user data for this file entry. */
	FSharedBufferView CustomData;

	UE_API FFileEntry(FBlobHandleWithHash InTarget, FUtf8String InName, EFileEntryFlags InFlags, int64 InLength, const FIoHash& InHash, FSharedBufferView InCustomData);
	UE_API ~FFileEntry();

	static UE_API FFileEntry Read(FBlobReader& Reader);
	UE_API void Write(FBlobWriter& Writer) const;
};

#undef UE_API
