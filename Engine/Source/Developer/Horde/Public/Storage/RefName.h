// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Utf8String.h"

#define UE_API HORDE_API

// Identifier for a ref in the storage system. Refs serve as GC roots, and are persistent entry points to expanding data structures within the store.
struct FRefName
{
public:
	UE_API FRefName(FUtf8String Text);
	UE_API ~FRefName();

	/** Accessor for the underlying string. */
	UE_API const FUtf8String& GetText() const;

	UE_API bool operator==(const FRefName& Other) const;
	UE_API bool operator!=(const FRefName& Other) const;
	friend uint32 GetTypeHash(const FRefName& RefName);

private:
	FUtf8String Text;
};

#undef UE_API
