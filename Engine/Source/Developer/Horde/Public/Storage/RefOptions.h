// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API HORDE_API

/**
 * Options for adding a new ref
 */ 
struct FRefOptions
{
	static UE_API const FRefOptions Default;

	/** Time until a ref is expired. If zero, the ref does not expire. */
	FTimespan Lifetime;

	/** Whether to extend the remaining lifetime of a ref whenever it is fetched. Defaults to true. */
	bool bExtend = true;

	UE_API FRefOptions();
};

#undef UE_API
