// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Blob.h"
#include "BlobHandle.h"
#include "BlobLocator.h"
#include "RefName.h"
#include "Templates/SharedPointer.h"

#define UE_API HORDE_API

class FBlobAlias;
struct FAliasInfo;
class FBlobWriter;
struct FRefOptions;

/**
 * Specifies a relative or absolute time value for returning cached ref values
 */
struct FRefCacheTime
{
	FDateTime Time;

	FRefCacheTime();
	FRefCacheTime(FTimespan InMaxAge);
	FRefCacheTime(FDateTime InTime);
};

/**
 * Interface for the storage system.
 */
class FStorageClient : public TSharedFromThis<FStorageClient, ESPMode::ThreadSafe>
{
public:
	UE_API virtual ~FStorageClient();

	//
	// Blobs
	//

	/** Creates a new blob handle from a blob locator. */
	virtual FBlobHandle CreateHandle(const FBlobLocator& Locator) const = 0;

	/** Creates a new writer for storage blobs. */
	virtual TUniquePtr<FBlobWriter> CreateWriter(FUtf8String BasePath = FUtf8String()) = 0;

	//
	// Aliases
	//

	/** Adds an alias to a given blob. */
	virtual void AddAlias(const char* Name, FBlobAlias Alias) = 0;

	/** Removes an alias from a blob. */
	virtual void RemoveAlias(const char* Name, FBlobHandle Handle) = 0;

	/** Finds blobs with the given alias. Unlike refs, aliases do not serve as GC roots. */
	virtual void FindAliases(const char* Name, int MaxResults, TArray<FBlobAlias>& OutAliases) = 0;

	//
	// Refs
	//

	/** Deletes a ref. */
	virtual bool DeleteRef(const FRefName& Name) = 0;

	/** Reads data for a ref from the store. */
	virtual FBlobHandle ReadRef(const FRefName& Name, const FRefCacheTime& CacheTime) const = 0;

	/** Writes a new ref to the store. */
	UE_API void WriteRef(const FRefName& Name, const FBlobHandle& Handle);
	virtual void WriteRef(const FRefName& Name, const FBlobHandle& Handle, const FRefOptions& Options) = 0;

protected:

	/** Creates a new locator with a particular base path. */
	static UE_API FBlobLocator CreateLocator(const FUtf8StringView& BasePath);
};

/**
 * Target for an aliased blob name
 */
class FBlobAlias
{
public:
	/** Handle to the target blob for the alias. */
	FBlobHandle Target;

	/** Rank for the alias. */
	int Rank;

	/** Data stored inline with the alias. */
	FSharedBufferView Data;

	FBlobAlias(FBlobHandle InTarget, int InRank, FSharedBufferView InData);
	~FBlobAlias();
};

/** 
 * Alias for newly added blobs
 */
struct FAliasInfo
{
	/** Name of the alias. */
	FUtf8String Name;

	/** Rank for ordering aliases (lower values are preferred). */
	int Rank;

	/** Inline data to be stored with the alias. */
	FSharedBufferView Data;

	FAliasInfo(FUtf8String InName, int InRank = 0);
	FAliasInfo(FUtf8String InName, int InRank, FSharedBufferView InData);
	~FAliasInfo();
};

#undef UE_API
